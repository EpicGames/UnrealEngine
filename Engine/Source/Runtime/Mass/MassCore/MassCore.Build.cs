// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MassCore : ModuleRules
	{
		public MassCore(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			// Headers live in Public/Mass/ — add to include paths so bare
			// #include "EntityHandle.h" resolves for direct consumers.
			PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "Public"));

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"TraceLog",
				}
			);
		}
	}
}
