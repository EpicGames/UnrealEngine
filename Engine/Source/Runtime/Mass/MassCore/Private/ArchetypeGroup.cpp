// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mass/ArchetypeGroup.h"


namespace UE::Mass
{
	//-----------------------------------------------------------------------------
	// FArchetypeGroups
	//-----------------------------------------------------------------------------
	bool FArchetypeGroups::operator==(const FArchetypeGroups& OtherGroups) const
	{
		return IDContainer == OtherGroups.IDContainer;
	}

	FArchetypeGroups& FArchetypeGroups::operator=(FArchetypeGroups&& InGroups)
	{
		IDContainer = MoveTemp(InGroups.IDContainer);
		return *this;
	}

	FArchetypeGroups& FArchetypeGroups::operator=(const FArchetypeGroups& InGroups)
	{
		IDContainer = InGroups.IDContainer;
		return *this;
	}

	void FArchetypeGroups::Add(FArchetypeGroupHandle GroupHandle)
	{
		const int32 GroupTypeIndex = static_cast<int32>(GroupHandle.GetGroupType());
		if (IDContainer.IsValidIndex(GroupTypeIndex))
		{
			IDContainer[GroupTypeIndex] = GroupHandle.GetGroupID();
		}
		else
		{
			IDContainer.EmplaceAt(GroupTypeIndex, GroupHandle.GetGroupID());
		}
	}

	FArchetypeGroups FArchetypeGroups::Add(FArchetypeGroupHandle GroupHandle) const
	{
		FArchetypeGroups Copy = *this;
		const int32 GroupTypeIndex = static_cast<int32>(GroupHandle.GetGroupType());
		if (IDContainer.IsValidIndex(GroupTypeIndex))
		{
			Copy.IDContainer[GroupTypeIndex] = GroupHandle.GetGroupID();
		}
		else
		{
			Copy.IDContainer.EmplaceAt(GroupTypeIndex, GroupHandle.GetGroupID());
		}

		return MoveTemp(Copy);
	}

	void FArchetypeGroups::Remove(FArchetypeGroupType GroupType)
	{
		const int32 GroupTypeIndex = static_cast<int32>(GroupType);
		if (IDContainer.IsValidIndex(GroupTypeIndex))
		{
			const bool bIsLastElement = ((IDContainer.GetMaxIndex() - 1) == GroupTypeIndex);
			IDContainer.RemoveAtUninitialized(GroupTypeIndex);
			if (bIsLastElement)
			{
				Shrink();
			}
		}
	}

	FArchetypeGroups FArchetypeGroups::Remove(FArchetypeGroupType GroupType) const
	{
		const int32 GroupTypeIndex = static_cast<int32>(GroupType);
		if (IDContainer.IsValidIndex(GroupTypeIndex))
		{
			FArchetypeGroups Copy = *this;
			Copy.IDContainer.RemoveAtUninitialized(GroupTypeIndex);

			// if we removed the last element we need to shrink the container too.
			if (const bool bIsLastElement = ((IDContainer.GetMaxIndex() - 1) == GroupTypeIndex))
			{
				Copy.Shrink();
			}

			return MoveTemp(Copy);
		}
		return *this;
	}

	void FArchetypeGroups::Shrink()
	{
		IDContainer.Shrink();
	}

	bool FArchetypeGroups::IsShrunk() const
	{
		const int32 MaxIndex = IDContainer.GetMaxIndex();
		// if the IDContainer has been shrunk, or never needed to be shrunk, the last element in the available
		// container is valid.
		return MaxIndex == 0 || IDContainer.IsValidIndex(MaxIndex - 1) ;
	}

	uint32 GetTypeHash(const FArchetypeGroups& Instance)
	{
		uint32 Hash = 0;

		for (auto It = Instance.IDContainer.CreateConstIterator(); It; ++It)
		{
			const FArchetypeGroupHandle GroupHandle(FArchetypeGroupType(It.GetIndex()), Instance.IDContainer[It.GetIndex()]);
			Hash = HashCombine(Hash, GetTypeHash(GroupHandle));
		}

		return Hash;
	}
} // namespace UE::Mass
