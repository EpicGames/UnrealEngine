// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mass/TestableEnsures.h"

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST

DEFINE_LOG_CATEGORY(LogMassTestableEnsures);

namespace UE::Mass::Test
{
	int32 TestsInProgress = 0;
}

#endif // !UE_BUILD_SHIPPING && !UE_BUILD_TEST
