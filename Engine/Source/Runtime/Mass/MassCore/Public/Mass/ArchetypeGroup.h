// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/Map.h"


namespace UE::Mass
{
	//-----------------------------------------------------------------------------
	// FArchetypeGroupType
	//-----------------------------------------------------------------------------
	struct FArchetypeGroupType
	{
		constexpr static uint32 InvalidArchetypeGroupTypeIdentifier = static_cast<uint32>(INDEX_NONE);

		explicit FArchetypeGroupType(const uint32 Value = InvalidArchetypeGroupTypeIdentifier)
			: Identifier(Value)
		{
		}

		FArchetypeGroupType(const FArchetypeGroupType& Source) = default;

		friend uint32 GetTypeHash(const FArchetypeGroupType& Instance)
		{
			return GetTypeHash(Instance.Identifier);
		}

		bool operator==(const FArchetypeGroupType Other) const
		{
			return Identifier == Other.Identifier;
		}

		bool operator<(const FArchetypeGroupType Other) const
		{
			return Identifier < Other.Identifier;
		}

		explicit operator int32() const
		{
			return static_cast<int32>(Identifier);
		}

		bool IsValid() const
		{
			return Identifier != InvalidArchetypeGroupTypeIdentifier;
		}

	private:
		uint32 Identifier;
	};


	struct FArchetypeGroupID
	{
		constexpr static uint32 InvalidArchetypeGroupID = static_cast<uint32>(INDEX_NONE);
		constexpr static uint32 FirstGroupID = 0;

		FArchetypeGroupID() = default;
		FArchetypeGroupID(const uint32 InID)
			: ID(InID)
		{
		}

		bool operator==(const FArchetypeGroupID Other) const
		{
			return ID == Other.ID;
		}

		bool IsValid() const
		{
			return ID != InvalidArchetypeGroupID;
		}

		operator int32() const
		{
			return static_cast<int32>(ID);
		}

		static FArchetypeGroupID First()
		{
			return FArchetypeGroupID(FirstGroupID);
		}

		FArchetypeGroupID Next() const
		{
			return FArchetypeGroupID(ID + 1);
		}

	private:
		uint32 ID = InvalidArchetypeGroupID;
	};

	//-----------------------------------------------------------------------------
	// FArchetypeGroupHandle
	//-----------------------------------------------------------------------------
	struct FArchetypeGroupHandle
	{
		explicit FArchetypeGroupHandle(const FArchetypeGroupType InGroupType, const FArchetypeGroupID InGroupID)
			: GroupType(InGroupType), GroupID(InGroupID)
		{
		}

		FArchetypeGroupHandle() = default;

		bool operator==(const FArchetypeGroupHandle Other) const
		{
			return GroupType == Other.GroupType && GroupID == Other.GroupID;
		}

		bool operator!=(const FArchetypeGroupHandle Other) const
		{
			return !(*this == Other);
		}

		friend uint32 GetTypeHash(const FArchetypeGroupHandle& Instance)
		{
			const uint64 CombinedHandle = static_cast<uint64>(static_cast<int32>(Instance.GroupType)) << 32 | Instance.GroupID;
			return GetTypeHash(CombinedHandle);
		}

		FArchetypeGroupType GetGroupType() const
		{
			return GroupType;
		}

		FArchetypeGroupID GetGroupID() const
		{
			return GroupID;
		}

		bool operator<(const FArchetypeGroupHandle Other) const
		{
			return GroupType < Other.GroupType || (GroupType == Other.GroupType && GroupID < Other.GroupID);
		}

		void UpdateID(const FArchetypeGroupHandle Other)
		{
			if (ensureMsgf(Other.GroupType == GroupType, TEXT("Updating ID is only supported for group handles of the same type")))
			{
				new (this) FArchetypeGroupHandle(GroupType, Other.GroupID);
			}
		}

		bool IsValid() const
		{
			return GroupType.IsValid() && GroupID.IsValid();
		}

	private:
		FArchetypeGroupType GroupType;
		FArchetypeGroupID GroupID;
	};

	//-----------------------------------------------------------------------------
	// FArchetypeGroups
	//-----------------------------------------------------------------------------
	struct FArchetypeGroups
	{
		FArchetypeGroups() = default;
		FArchetypeGroups(const FArchetypeGroups& InGroups) = default;
		FArchetypeGroups(FArchetypeGroups&& InGroups) = default;

		MASSCORE_API bool operator==(const FArchetypeGroups& OtherGroups) const;
		MASSCORE_API FArchetypeGroups& operator=(FArchetypeGroups&& InGroups);
		MASSCORE_API FArchetypeGroups& operator=(const FArchetypeGroups& InGroups);

		/**
		 * Adds or updates the given (GroupType, GroupID) combination to IDContainer
		 */
		MASSCORE_API void Add(FArchetypeGroupHandle GroupHandle);

		/**
		 * Adds or updates the given (GroupType, GroupID) combination to IDContainer
		 * @return a copy of this FArchetypeGroups container with GroupHandle added to the ID container
		 * @note using [[nodiscard]] to avoid accidental calls on const instances that would not produce any effects
		 */
		[[nodiscard]] MASSCORE_API FArchetypeGroups Add(FArchetypeGroupHandle GroupHandle) const;

		/**
		 * Removes the stored GroupID associated with the given GroupType.
		 * If the given group type is not stored in IDContainer the request is ignored.
		 */
		MASSCORE_API void Remove(FArchetypeGroupType GroupType);

		/**
		 * Removes the stored GroupID associated with the given GroupType.
		 * If the given group type is not stored in IDContainer the request is ignored.
		 * @return a copy of this FArchetypeGroups container with GroupType removed.
		 * @note using [[nodiscard]] to avoid accidental calls on const instances that would not produce any effects
		 */
		[[nodiscard]] MASSCORE_API FArchetypeGroups Remove(FArchetypeGroupType GroupType) const;

		MASSCORE_API void Shrink();
		MASSCORE_API bool IsShrunk() const;

		FArchetypeGroupID GetID(const FArchetypeGroupType GroupType) const
		{
			return IDContainer.IsValidIndex(static_cast<int32>(GroupType)) ? IDContainer[static_cast<int32>(GroupType)] : FArchetypeGroupID();
		}

		bool ContainsType(const FArchetypeGroupType GroupType) const
		{
			return IDContainer.IsValidIndex(static_cast<int32>(GroupType));
		}

		friend MASSCORE_API uint32 GetTypeHash(const FArchetypeGroups& Instance);

	protected:
		TSparseArray<FArchetypeGroupID> IDContainer;
	};

} // namespace UE::Mass
