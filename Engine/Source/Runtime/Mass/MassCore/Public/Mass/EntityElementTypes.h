// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntityElementTypes.generated.h"

USTRUCT()
struct FMassElement
{
	GENERATED_BODY()
};

/** This is the base class for all Mass fragments */
USTRUCT()
struct FMassFragment : public FMassElement
{
	GENERATED_BODY()
};

// these are the messages we'll print out when static checks whether a given type is a fragment fails
#define _MASS_INVALID_FRAGMENT_CORE_MESSAGE "Make sure to inherit from FMassFragment or one of its child-types and ensure that the struct is trivially copyable, or opt out by specializing TMassFragmentTraits for this type and setting AuthorAcceptsItsNotTriviallyCopyable = true"
#define MASS_INVALID_FRAGMENT_MSG  "Given struct doesn't represent a valid fragment type." _MASS_INVALID_FRAGMENT_CORE_MESSAGE
#define MASS_INVALID_FRAGMENT_MSG_F  "Type %s is not a valid fragment type." _MASS_INVALID_FRAGMENT_CORE_MESSAGE

/**
 * Special-case base class for fragments. Derive from this if you intend to use the fragment type as a sparse fragment
 * See UE::Mass::FSparseElementsStorage for more details on sparse elements.
 */
USTRUCT()
struct FMassSparseFragment : public FMassFragment
{
	GENERATED_BODY()
};

/**
 * This is the base class for types that will only be tested for presence/absence, i.e. Tags.
 * Subclasses should never contain any member properties.
 */
USTRUCT()
struct FMassTag : public FMassElement
{
	GENERATED_BODY()
};

/**
 * Special-case base class for tags. Derive from this if you intend to use the tag type as a sparse tag
 * See UE::Mass::FSparseElementsStorage for more details on sparse elements.
 */
USTRUCT()
struct FMassSparseTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassChunkFragment : public FMassElement
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassSharedFragment : public FMassElement
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassConstSharedFragment : public FMassElement
{
	GENERATED_BODY()
};

namespace UE::Mass
{
	template<typename T>
	bool IsA(const UStruct* /*Struct*/)
	{
		return false;
	}

	template<>
	inline bool IsA<FMassElement>(const UStruct* Struct)
	{
		// @todo check that it's not any of the directly inherited types (that are "abstract" in their nature until inherited from themselves). 
		return Struct && Struct->IsChildOf(FMassElement::StaticStruct());
	}

	template<>
	inline bool IsA<FMassFragment>(const UStruct* Struct)
	{
		return Struct && Struct->IsChildOf(FMassFragment::StaticStruct());
	}

	template<>
	inline bool IsA<FMassSparseFragment>(const UStruct* Struct)
	{
		return Struct && Struct->IsChildOf(FMassSparseFragment::StaticStruct());
	}

	template<>
	inline bool IsA<FMassTag>(const UStruct* Struct)
	{
		return Struct && Struct->IsChildOf(FMassTag::StaticStruct());
	}

	template<>
	inline bool IsA<FMassSparseTag>(const UStruct* Struct)
	{
		return Struct && Struct->IsChildOf(FMassSparseTag::StaticStruct());
	}

	template<>
	inline bool IsA<FMassChunkFragment>(const UStruct* Struct)
	{
		return Struct && Struct->IsChildOf(FMassChunkFragment::StaticStruct());
	}

	template<>
	inline bool IsA<FMassSharedFragment>(const UStruct* Struct)
	{
		return Struct && Struct->IsChildOf(FMassSharedFragment::StaticStruct());
	}

	template<>
	inline bool IsA<FMassConstSharedFragment>(const UStruct* Struct)
	{
		return Struct && Struct->IsChildOf(FMassConstSharedFragment::StaticStruct());
	}

	inline bool IsSparse(TNotNull<const UStruct*> Struct)
	{
		return Struct->IsChildOf(FMassSparseFragment::StaticStruct()) || Struct->IsChildOf(FMassSparseTag::StaticStruct());
	}
} // namespace UE::Mass
