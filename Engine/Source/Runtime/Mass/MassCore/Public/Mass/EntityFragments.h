// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityElementTypes.h"

#include "EntityFragments.generated.h"

USTRUCT()
struct FTransformFragment : public FMassFragment
{
	GENERATED_BODY()

	FTransformFragment() = default;
	FTransformFragment(const FTransform& InTransform)
		: Transform(InTransform)
	{
	}

	const FTransform& GetTransform() const
	{
		return Transform;
	}

	void SetTransform(const FTransform& InTransform)
	{
		Transform = InTransform;
	}

	FTransform& GetMutableTransform()
	{
		return Transform;
	}

protected:
	UPROPERTY()
	FTransform Transform;
};

