// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntityHandle.generated.h"

/**
 * A handle to a Mass entity. An entity is used in conjunction with the FMassEntityManager
 * for the current world and can contain lightweight fragments.
 */
USTRUCT(BlueprintType)
struct alignas(8) FMassEntityHandle
{
	GENERATED_BODY()

	FMassEntityHandle() = default;
	FMassEntityHandle(const int32 InIndex, const int32 InSerialNumber)
		: Index(InIndex), SerialNumber(InSerialNumber)
	{
	}
	
	UPROPERTY(VisibleAnywhere, Category = "Mass|Debug", Transient)
	int32 Index = 0;
	
	UPROPERTY(VisibleAnywhere, Category = "Mass|Debug", Transient)
	int32 SerialNumber = 0;

	bool operator==(const FMassEntityHandle Other) const
	{
		return Index == Other.Index && SerialNumber == Other.SerialNumber;
	}

	bool operator!=(const FMassEntityHandle Other) const
	{
		return !operator==(Other);
	}

	/** Has meaning only for sorting purposes */
	bool operator<(const FMassEntityHandle Other) const
	{
		return Index < Other.Index;
	}

	/** Note that this function is merely checking if Index and SerialNumber are set. There's no way to validate if 
	 *  these indicate a valid entity in an EntitySubsystem without asking the system. */
	bool IsSet() const
	{
		return Index != 0 && SerialNumber != 0;
	}

	inline bool IsValid() const
	{
		return IsSet();
	}

	void Reset()
	{
		Index = SerialNumber = 0;
	}

	/** Allows the entity handle to be shared anonymously. */
	// Relying on the fact that this struct only stores 2 integers and is aligned correctly.
	uint64 AsNumber() const
	{
		return *reinterpret_cast<const uint64*>(this);
	}
	/** Reconstruct the entity handle from an anonymously shared integer. */
	static FMassEntityHandle FromNumber(uint64 Value) 
	{
		FMassEntityHandle Result;
		*reinterpret_cast<uint64*>(&Result) = Value;
		return Result;
	}

	friend uint32 GetTypeHash(const FMassEntityHandle Entity)
	{
		return HashCombine(Entity.Index, Entity.SerialNumber);
	}

	friend FString LexToString(const FMassEntityHandle Entity)
	{
		return Entity.DebugGetDescription();
	}

	FString DebugGetDescription() const
	{
		return FString::Printf(TEXT("i: %d sn: %d"), Index, SerialNumber);
	}
};

static_assert(sizeof(FMassEntityHandle) == sizeof(uint64), "Expected FMassEntityHandle to be convertible to a 64-bit integer value, so size needs to be 8 bytes.");
static_assert(alignof(FMassEntityHandle) == sizeof(uint64), "Expected FMassEntityHandle to be convertible to a 64-bit integer value, so alignment needs to be 8 bytes.");
