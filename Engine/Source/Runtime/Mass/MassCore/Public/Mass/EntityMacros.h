// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructUtilsMacros.h"

#ifndef WITH_MASSENTITY_DEBUG
#define WITH_MASSENTITY_DEBUG (!(UE_BUILD_SHIPPING || UE_BUILD_SHIPPING_WITH_EDITOR || UE_BUILD_TEST) && WITH_STRUCTUTILS_DEBUG && 1)
#endif
