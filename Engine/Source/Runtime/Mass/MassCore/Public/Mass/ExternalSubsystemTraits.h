// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Engine/World.h"
#include "Subsystems/WorldSubsystem.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

/**
 * Traits describing how a given piece of code can be used by Mass. We require author or user of a given subsystem to 
 * define its traits. To do it add the following in an accessible location. 
 *
 * template<>
 * struct TMassExternalSubsystemTraits<UMyCustomManager>
 * {
 *		enum { GameThreadOnly = false }
 * }
 *
 * this will let Mass know it can access UMyCustomManager on any thread.
 *
 * This information is being used to calculate processor and query dependencies as well as appropriate distribution of
 * calculations across threads.
 */
template <typename T>
struct TMassExternalSubsystemTraits final
{
	enum
	{
		// Unless configured otherwise each subsystem will be treated as "game-thread only".
		GameThreadOnly = true,

		// If set to true all RW and RO operations will be viewed as RO when calculating processor dependencies
		ThreadSafeWrite = !GameThreadOnly,
	};
};

/** 
 * Shared Fragment traits.
 * @see TMassExternalSubsystemTraits
 */
template <typename T>
struct TMassSharedFragmentTraits final
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
};

/** 
 * Fragment traits.
 * @see TMassExternalSubsystemTraits
 */
template <typename T>
struct TMassFragmentTraits final
{
	enum
	{
		// Fragment types are best kept trivially copyable for performance reasons.
		// To enforce that we test this trait when checking if a given type is a valid fragment type.
		// This test can be skipped by specifically opting out, which also documents that 
		// making the given type non-trivially-copyable was a deliberate decision. 
		AuthorAcceptsItsNotTriviallyCopyable = false
	};
};