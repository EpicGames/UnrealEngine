// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Self-contained testable assertion macros for Mass.
// These allow Mass runtime checks to log-and-continue during automated tests
// instead of crashing, while behaving as standard ensure/check in production.
//
// @todo: if other engine systems need testable ensures, consider extracting
// this to a standalone TestableAssertions module rather than duplicating.

#include "Misc/AssertionMacros.h"

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST

#include "Logging/LogCategory.h"

MASSCORE_API DECLARE_LOG_CATEGORY_EXTERN(LogMassTestableEnsures, Log, All);

namespace UE::Mass::Test
{
	// Counter indicating how many test scopes are active. When > 0, testable
	// macros log instead of crashing.
	MASSCORE_API extern int32 TestsInProgress;
}

#define testableEnsureMsgf(InExpression, InFormat, ... ) \
	(LIKELY(!!(InExpression)) || ([&]() \
		{ \
			if (UNLIKELY(UE::Mass::Test::TestsInProgress > 0)) \
			{ \
				UE_LOG(LogMassTestableEnsures, Warning, InFormat, ##__VA_ARGS__); \
			} \
			else \
			{ \
				ensureMsgf(InExpression, InFormat, ##__VA_ARGS__); \
			} \
		return false; \
		} ()))

#define testableCheckf(InExpression, InFormat, ... ) \
	if (UNLIKELY(UE::Mass::Test::TestsInProgress > 0)) \
	{ \
		if (!(InExpression))\
		{ \
			UE_LOG(LogMassTestableEnsures, Error, InFormat, ##__VA_ARGS__); \
			return; \
		}\
	} \
	else \
	{ \
		checkf(InExpression, InFormat, ##__VA_ARGS__); \
	}

#define testableCheckfReturn(InExpression, ReturnExpression, InFormat, ... ) \
	if (UNLIKELY(UE::Mass::Test::TestsInProgress > 0)) \
	{ \
		if (!(InExpression)) \
		{ \
			UE_LOG(LogMassTestableEnsures, Error, InFormat, ##__VA_ARGS__); \
			ReturnExpression; \
		} \
	} \
	else \
	{ \
		checkf(InExpression, InFormat, ##__VA_ARGS__); \
	}

#else // UE_BUILD_SHIPPING || UE_BUILD_TEST

struct FMassTestableEnsureScope {};

#define testableEnsureMsgf ensureMsgf
#define testableCheckf checkf
#define testableCheckfReturn(InExpression, ReturnValue, InFormat, ... ) checkf(InExpression, InFormat, ##__VA_ARGS__)

#endif
