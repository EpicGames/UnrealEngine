// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassDeveloper : ModuleRules
	{
		public static bool IsMassTraceAnalysisEnabled(ReadOnlyTargetRules Target)
		{
			return Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop)
				&& (Target.Type == TargetType.Editor || Target.Type == TargetType.Program);
		}

		public MassDeveloper(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			});

			if (IsMassTraceAnalysisEnabled(Target))
			{
				PublicDependencyModuleNames.Add("TraceServices");

				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"TraceLog",
						"TraceAnalysis",
					});

				PublicDefinitions.Add("UE_MASS_TRACE_ANALYSIS_ENABLED=1");
			}
			else
			{
				PublicDefinitions.Add("UE_MASS_TRACE_ANALYSIS_ENABLED=0");
			}
		}
	}
}
