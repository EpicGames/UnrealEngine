// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/MassTraceAnalyzer.h"

#if UE_MASS_TRACE_ANALYSIS_ENABLED

#include "Common/ProviderLock.h"
#include "HAL/LowLevelMemTracker.h"
#include "Trace/MassTraceProvider.h"
#include "Trace/MassTraceTypes.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace UE::Mass::Trace
{

FMassTraceAnalyzer::FMassTraceAnalyzer(TraceServices::IAnalysisSession& InSession, TSharedRef<FMassTraceProvider> InProvider)
	: Session(InSession)
	, MassTraceProvider(InProvider)
{
}

void FMassTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	FInterfaceBuilder& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_RegisterMassFragment, "MassTrace", "RegisterMassFragment");
	Builder.RouteEvent(RouteId_RegisterMassArchetype, "MassTrace", "RegisterMassArchetype");

	Builder.RouteEvent(RouteId_MassBulkAddEntity, "MassTrace", "MassBulkAddEntity");
	Builder.RouteEvent(RouteId_MassEntityMoved, "MassTrace", "MassEntityMoved");
	Builder.RouteEvent(RouteId_MassBulkEntityDestroyed, "MassTrace", "MassBulkEntityDestroyed");

	Builder.RouteEvent(RouteId_MassPhaseBegin, "MassTrace", "MassPhaseBegin");
	Builder.RouteEvent(RouteId_MassPhaseEnd, "MassTrace", "MassPhaseEnd");
}

void FMassTraceAnalyzer::OnAnalysisEnd()
{
	TraceServices::FProviderEditScopeLock RegionProviderScopedLock(MassTraceProvider.Get());
	MassTraceProvider.Get().OnAnalysisSessionEnded();
}

bool FMassTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	const FEventData& EventData = Context.EventData;
	switch (RouteId)
	{
	case RouteId_RegisterMassFragment:
	{
		const uint64 FragmentId = EventData.GetValue<uint64>("FragmentId");
		FString FragmentName;
		EventData.GetString("FragmentName", FragmentName);

		const uint32 FragmentSize = EventData.GetValue<uint32>("FragmentSize");
		const EFragmentType FragmentType = static_cast<EFragmentType>(EventData.GetValue<uint8>("FragmentType"));
		const FFragmentInfo FragmentInfo
		{
			.Id = FragmentId,
			.Name = MoveTemp(FragmentName),
			.Size = FragmentSize,
			.Type = FragmentType,
		};

		TraceServices::FProviderEditScopeLock Lock(MassTraceProvider.Get());

		MassTraceProvider.Get().AddFragment(FragmentInfo);
		break;
	}
	case RouteId_RegisterMassArchetype:
	{
		const uint64 Id = EventData.GetValue<uint64>("ArchetypeID");
		const TArrayView<const uint64> FragmentArrayView = EventData.GetArrayView<uint64>("Fragments");
		FArchetypeInfo ArchetypeInfo
		{
			.Id = Id,
		};

		ArchetypeInfo.Fragments.Reserve(FragmentArrayView.Num());
		{
			TraceServices::FProviderReadScopeLock ReadLock(MassTraceProvider.Get());
			for (int32 Index = 0; Index < FragmentArrayView.Num(); ++Index)
			{
				const uint64 FragmentId = FragmentArrayView[Index];
				if (const FFragmentInfo* Info = MassTraceProvider.Get().FindFragmentById(FragmentId))
				{
					ArchetypeInfo.Fragments.Add(Info);
				}
			}
			Algo::Sort(ArchetypeInfo.Fragments, [](const FFragmentInfo* Lhs, const FFragmentInfo* Rhs)
				{
					if (Lhs->Type == Rhs->Type)
					{
						return Lhs->Name < Rhs->Name;
					}
					return Lhs->Type < Rhs->Type;
				});
		}
		{
			TraceServices::FProviderEditScopeLock EditLock(MassTraceProvider.Get());
			MassTraceProvider.Get().AddArchetype(ArchetypeInfo);
		}
		break;
	}
	case RouteId_MassBulkAddEntity:
	{
		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		const double ProfileTime = Context.EventTime.AsSeconds(Cycle);
		// RecordingTime field was added later; old traces return -1.0 (missing field sentinel).
		// Fall back to ProfileTime which gives the old behavior (not pause-aligned but functional).
		const double RecordingTime = EventData.GetValue<double>("RecordingTime", -1.0);
		const double EffectiveTime = RecordingTime >= 0.0 ? RecordingTime : ProfileTime;
		const TArrayView<const uint64> Entities = EventData.GetArrayView<uint64>("Entities");
		const TArrayView<const uint64> ArchetypeIDs = EventData.GetArrayView<uint64>("ArchetypeIDs");

		TraceServices::FProviderEditScopeLock EditLock(MassTraceProvider.Get());
		MassTraceProvider.Get().BulkAddEntity(EffectiveTime, ProfileTime, Entities, ArchetypeIDs);
		break;
	}
	case RouteId_MassEntityMoved:
	{
		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		const double ProfileTime = Context.EventTime.AsSeconds(Cycle);
		const double RecordingTime = EventData.GetValue<double>("RecordingTime", -1.0);
		const double EffectiveTime = RecordingTime >= 0.0 ? RecordingTime : ProfileTime;
		const uint64 Entity = EventData.GetValue<uint64>("Entity");
		const uint64 Archetype = EventData.GetValue<uint64>("NewArchetypeID");

		TraceServices::FProviderEditScopeLock EditLock(MassTraceProvider.Get());
		MassTraceProvider.Get().BulkMoveEntity(
			EffectiveTime,
			ProfileTime,
			MakeConstArrayView(&Entity, 1),
			MakeConstArrayView(&Archetype, 1));
		break;
	}
	case RouteId_MassBulkEntityDestroyed:
	{
		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		const double ProfileTime = Context.EventTime.AsSeconds(Cycle);
		const double RecordingTime = EventData.GetValue<double>("RecordingTime", -1.0);
		const double EffectiveTime = RecordingTime >= 0.0 ? RecordingTime : ProfileTime;
		const TArrayView<const uint64> Entities = EventData.GetArrayView<uint64>("Entities");

		TraceServices::FProviderEditScopeLock EditLock(MassTraceProvider.Get());
		MassTraceProvider.Get().BulkDestroyEntity(
			EffectiveTime,
			ProfileTime,
			Entities);
		break;
	}
	case RouteId_MassPhaseBegin:
	{
		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		FString PhaseName;
		EventData.GetString("PhaseName", PhaseName);

		const uint64 PhaseId = EventData.GetValue<uint64>("PhaseId", 0);
		// a missing or 0 RegionID indicates that the region is identified by name only
		if (PhaseId > 0)
		{
			TraceServices::FProviderEditScopeLock RegionProviderScopedLock(MassTraceProvider.Get());
			MassTraceProvider.Get().AppendRegionBegin(*PhaseName, PhaseId, Context.EventTime.AsSeconds(Cycle));
		}
		else
		{
			TraceServices::FProviderEditScopeLock RegionProviderScopedLock(MassTraceProvider.Get());
			MassTraceProvider.Get().AppendRegionBegin(*PhaseName, Context.EventTime.AsSeconds(Cycle));
		}

		break;
	}

	case RouteId_MassPhaseEnd:
	{
		const uint64 Cycle = EventData.GetValue<uint64>("Cycle");
		const uint64 PhaseID = EventData.GetValue<uint64>("PhaseId", 0);

		TraceServices::FProviderEditScopeLock RegionProviderScopedLock(MassTraceProvider.Get());
		if (PhaseID > 0)
		{
			MassTraceProvider.Get().AppendRegionEnd(PhaseID, Context.EventTime.AsSeconds(Cycle));
		}
		else
		{
			FString PhaseName = TEXT("Invalid");
			EventData.GetString("PhaseName", PhaseName);
			MassTraceProvider.Get().AppendRegionEnd(*PhaseName, Context.EventTime.AsSeconds(Cycle));
		}
		break;
	}
	default:
		break;
	}

	return true;
}

} // namespace UE::Mass::Trace

#endif // UE_MASS_TRACE_ANALYSIS_ENABLED
