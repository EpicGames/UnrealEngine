// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_MASS_TRACE_ANALYSIS_ENABLED

#include "Templates/SharedPointer.h"
#include "Trace/Analyzer.h"

namespace TraceServices
{
class IAnalysisSession;
}

namespace UE::Mass::Trace
{
class FMassTraceProvider;

/**
 * Analyzer for Mass trace events, parses the trace events from the 'MassTrace' Logger.
 */
class FMassTraceAnalyzer
	: public UE::Trace::IAnalyzer
{
public:
	FMassTraceAnalyzer(TraceServices::IAnalysisSession& InSession, TSharedRef<FMassTraceProvider> InProvider);
	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_RegisterMassFragment,
		RouteId_RegisterMassArchetype,
		RouteId_MassBulkAddEntity,
		RouteId_MassEntityMoved,
		RouteId_MassBulkEntityDestroyed,
		RouteId_MassPhaseBegin,
		RouteId_MassPhaseEnd
	};

	TraceServices::IAnalysisSession& Session;
	TSharedRef<FMassTraceProvider> MassTraceProvider;
};

} // namespace UE::Mass::Trace

#endif // UE_MASS_TRACE_ANALYSIS_ENABLED
