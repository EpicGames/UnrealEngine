// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassTraceProvider.h"

#if UE_MASS_TRACE_ANALYSIS_ENABLED

#include "Algo/ForEach.h"
#include "Internationalization/Internationalization.h"
#include "Templates/ValueOrError.h"
#include "Trace/MassTraceAnalyzer.h"

DEFINE_LOG_CATEGORY(LogMassTrace)

namespace UE::Mass::Trace
{
static constexpr uint64 EntityEventsPageSize = 65536;

thread_local TraceServices::FProviderLock::FThreadLocalState GMassInsightsProviderLockState;

FMassTraceProvider::FMassTraceProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
	, EntityEvents(InSession.GetLinearAllocator(), EntityEventsPageSize)
{
}

int32 FMassTraceProvider::GetFragmentCount() const
{
	ReadAccessCheck();
	return FragmentInfoByID.Num();
}

const FFragmentInfo* FMassTraceProvider::FindFragmentById(const uint64 FragmentId) const
{
	ReadAccessCheck();

	const FFragmentInfo* const* Result = FragmentInfoByID.Find(FragmentId);
	if (Result != nullptr)
	{
		return *Result;
	}
	return nullptr;
}

const FArchetypeInfo* FMassTraceProvider::FindArchetypeById(const uint64 ArchetypeId) const
{
	ReadAccessCheck();

	const FArchetypeInfo* const* Result = ArchetypeByID.Find(ArchetypeId);
	if (Result != nullptr)
	{
		return *Result;
	}
	return nullptr;
}

void FMassTraceProvider::EnumerateFragments(const TFunctionRef<void(const FFragmentInfo& FragmentInfo, int32 Index)> Callback, const int32 BeginIndex) const
{
	ReadAccessCheck();
	for (int32 Index = BeginIndex, End = FragmentInfos.Num(); Index < End; Index++)
	{
		const FFragmentInfo& FragmentInfo = FragmentInfos[Index];
		Callback(FragmentInfo, Index);
	}
}

uint64 FMassTraceProvider::GetEntityEventCount() const
{
	ReadAccessCheck();

	return EntityEvents.Num();
}

TValueOrError<FEntityEventRecord, void> FMassTraceProvider::GetEntityEvent(const uint64 EventIndex) const
{
	ReadAccessCheck();

	if (EventIndex < EntityEvents.Num())
	{
		return MakeValue(EntityEvents[EventIndex]);
	}

	return MakeError();
}

void FMassTraceProvider::EnumerateEntityEvents(
	const uint64 StartIndex,
	const uint64 Count,
	const TFunctionRef<void(const FEntityEventRecord&, uint64 EventIndex)> Callback) const
{
	ReadAccessCheck();

	const uint64 EndIndex = FPlatformMath::Min(StartIndex + Count, EntityEvents.Num());
	for (uint64 Index = StartIndex; Index < EndIndex; Index++)
	{
		Callback(EntityEvents[Index], Index);
	}
}

uint64 FMassTraceProvider::GetRegionCount() const
{
	ReadAccessCheck();

	uint64 RegionCount = 0;
	for (const FRegionLane& Lane : Lanes)
	{
		RegionCount += Lane.Num();
	}
	return RegionCount;
}

int32 FMassTraceProvider::GetLaneCount() const
{
	ReadAccessCheck();
	return Lanes.Num();
}

const FRegionLane* FMassTraceProvider::GetLane(const int32 Index) const
{
	ReadAccessCheck();

	if (Index < Lanes.Num())
	{
		return &(Lanes[Index]);
	}
	return nullptr;
}

void FMassTraceProvider::AppendRegionBegin(const TCHAR* Name, const double Time)
{
	EditAccessCheck();
	// Regions identified by Name don't have an ID
	AppendRegionBegin(Name, 0, Time);
}

void FMassTraceProvider::AddFragment(const FFragmentInfo& FragmentInfo)
{
	EditAccessCheck();

	if (FragmentInfoByID.Find(FragmentInfo.Id) == nullptr)
	{
		const uint64 Id = FragmentInfo.Id;
		const int32 AllocatedIndex = FragmentInfos.AddElement(FragmentInfo);
		const FFragmentInfo* AllocatedFragment = &FragmentInfos[AllocatedIndex];
		FragmentInfoByID.Add(Id, AllocatedFragment);
	}
}

void FMassTraceProvider::AddArchetype(const FArchetypeInfo& ArchetypeInfo)
{
	EditAccessCheck();

	if (ArchetypeByID.Find(ArchetypeInfo.Id) == nullptr)
	{
		const uint64 Id = ArchetypeInfo.Id;
		const int32 AllocatedIndex = ArchetypeInfos.AddElement(ArchetypeInfo);
		const FArchetypeInfo* AllocatedArchetype = &ArchetypeInfos[AllocatedIndex];
		ArchetypeByID.Add(Id, AllocatedArchetype);
	}
}

void FMassTraceProvider::BulkAddEntity(const double RecordingTime, const double ProfileTime, const TConstArrayView<uint64> Entities, const TConstArrayView<uint64> ArchetypeIDs)
{
	EditAccessCheck();

	FEntityEventRecord Row;
	Row.Time = RecordingTime;
	Row.ProfileTime = ProfileTime;
	Row.Operation = EEntityEventType::Created;

	for (int32 Index = 0, End = Entities.Num(); Index < End; Index++)
	{
		Row.ArchetypeID = ArchetypeIDs[Index];
		Row.Entity = Entities[Index];
		EntityEvents.EmplaceBack(Row);
	}
}

void FMassTraceProvider::BulkMoveEntity(const double RecordingTime, const double ProfileTime, const TConstArrayView<uint64> Entities, const TConstArrayView<uint64> ArchetypeIDs)
{
	EditAccessCheck();

	FEntityEventRecord Row;
	Row.Time = RecordingTime;
	Row.ProfileTime = ProfileTime;
	Row.Operation = EEntityEventType::ArchetypeChange;

	for (int32 Index = 0, End = Entities.Num(); Index < End; Index++)
	{
		Row.ArchetypeID = ArchetypeIDs[Index];
		Row.Entity = Entities[Index];
		EntityEvents.EmplaceBack(Row);
	}
}

void FMassTraceProvider::BulkDestroyEntity(const double RecordingTime, const double ProfileTime, const TConstArrayView<uint64> Entities)
{
	EditAccessCheck();

	FEntityEventRecord Row;
	Row.Time = RecordingTime;
	Row.ProfileTime = ProfileTime;
	Row.Operation = EEntityEventType::Destroyed;
	Row.ArchetypeID = 0;

	for (int32 Index = 0, End = Entities.Num(); Index < End; Index++)
	{
		Row.Entity = Entities[Index];
		EntityEvents.EmplaceBack(Row);
	}
}

void FMassTraceProvider::AppendRegionBegin(const TCHAR* Name, const uint64 ID, const double Time)
{
	EditAccessCheck();
	// lookup by ID if the region has an ID
	FRegion** OpenRegion = ID ? OpenRegionsByID.Find(ID) : OpenRegionsByName.Find(Name);

	if (OpenRegion)
	{
		++NumWarnings;
		if (NumWarnings <= MaxWarningMessages)
		{
			UE_LOGF(LogMassTrace, Warning, "[Regions] A region begin event (%ls) was encountered while a region with same name is already open.", Name)
		}
	}
	else
	{
		FRegion Region;
		Region.BeginTime = Time;
		Region.Text = Session.StoreString(Name);
		Region.ID = ID;
		Region.Depth = CalculateRegionDepth(Region);

		if (Region.Depth == Lanes.Num())
		{
			Lanes.Emplace(Session.GetLinearAllocator());
		}

		Lanes[Region.Depth].Regions.EmplaceBack(Region);
		FRegion* NewOpenRegion = &(Lanes[Region.Depth].Regions.Last());

		if (ID)
		{
			OpenRegionsByID.Add(Region.ID, NewOpenRegion);
		}
		else
		{
			OpenRegionsByName.Add(Region.Text, NewOpenRegion);
		}
		UpdateCounter++;
	}

	// Update session time
	{
		TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
		Session.UpdateDurationSeconds(Time);
	}
}

void FMassTraceProvider::AppendRegionEnd(const uint64 ID, const double Time)
{
	EditAccessCheck();

	FRegion** OpenRegionPos = OpenRegionsByID.Find(ID);
	FRegion* OpenRegion = nullptr;
	if (OpenRegionPos)
	{
		OpenRegion = *OpenRegionPos;
	}

	AppendRegionEndInternal(OpenRegion, Time);
}

void FMassTraceProvider::AppendRegionEnd(const TCHAR* Name, const double Time)
{
	EditAccessCheck();

	FRegion** OpenRegionPos = OpenRegionsByName.Find(Name);
	FRegion* OpenRegion = nullptr;
	if (OpenRegionPos)
	{
		OpenRegion = *OpenRegionPos;
	}
	else
	{
		++NumWarnings;
		if (NumWarnings <= MaxWarningMessages)
		{
			UE_LOGF(LogMassTrace, Warning, "[Regions] A region end event (%ls) was encountered without having seen a matching region start event first.", Name)
		}
	}

	AppendRegionEndInternal(OpenRegion, Time);
}

void FMassTraceProvider::AppendRegionEndInternal(FRegion* OpenRegion, const double Time)
{
	if (OpenRegion)
	{
		OpenRegion->EndTime = Time;

		if (OpenRegion->ID)
		{
			OpenRegionsByID.Remove(OpenRegion->ID);
		}
		OpenRegionsByName.Remove(OpenRegion->Text);
		UpdateCounter++;
	}

	// Update session time
	{
		TraceServices::FAnalysisSessionEditScope SessionEditScope(Session);
		Session.UpdateDurationSeconds(Time);
	}
}

void FMassTraceProvider::OnAnalysisSessionEnded()
{
	EditAccessCheck();

	auto PrintOpenRegionMessage = [this](const TPair<uint64, FRegion*>& Pair)
		{
			const FRegion* Region = Pair.Value;
			++NumWarnings;
			if (NumWarnings <= MaxWarningMessages)
			{
				UE_LOGF(LogMassTrace, Warning, "[Regions] A region begin event (%ls) was never closed.", Region->Text)
			}
		};
	Algo::ForEach(OpenRegionsByID, PrintOpenRegionMessage);
	Algo::ForEach(OpenRegionsByName, [this](const TPair<FStringView, FRegion*>& Pair)
		{
			const FRegion* Region = Pair.Value;
			++NumWarnings;
			if (NumWarnings <= MaxWarningMessages)
			{
				UE_LOGF(LogMassTrace, Warning, "[Regions] A region begin event (%ls) was never closed.", Region->Text)
			}
		});

	if (NumWarnings > 0 || NumErrors > 0)
	{
		UE_LOGF(LogMassTrace, Error, "[Regions] %u warnings; %u errors", NumWarnings, NumErrors);
	}

	const uint64 TotalRegionCount = GetRegionCount();
	UE_LOGF(LogMassTrace, Log, "[Regions] Analysis completed (%llu regions, %d lanes).", TotalRegionCount, Lanes.Num());
}

int32 FMassTraceProvider::CalculateRegionDepth(const FRegion& Region) const
{
	constexpr int32 DepthLimit = 100;

	int32 NewDepth = 0;

	// Find first free lane/depth
	while (NewDepth < DepthLimit)
	{
		if (!Lanes.IsValidIndex(NewDepth))
		{
			break;
		}

		const FRegion& LastRegion = Lanes[NewDepth].Regions.Last();
		if (LastRegion.EndTime <= Region.BeginTime)
		{
			break;
		}
		NewDepth++;
	}

	ensureMsgf(NewDepth < DepthLimit, TEXT("Regions are nested too deep."));

	return NewDepth;
}

void FMassTraceProvider::EnumerateLanes(const TFunctionRef<void(const FRegionLane&, int32)> Callback) const
{
	ReadAccessCheck();

	for (int32 LaneIndex = 0; LaneIndex < Lanes.Num(); ++LaneIndex)
	{
		Callback(Lanes[LaneIndex], LaneIndex);
	}
}

bool FMassTraceProvider::EnumerateRegions(const double IntervalStart, const double IntervalEnd, const TFunctionRef<bool(const FRegion&)> Callback) const
{
	ReadAccessCheck();

	if (IntervalStart > IntervalEnd)
	{
		return false;
	}

	for (const FRegionLane& Lane : Lanes)
	{
		if (!Lane.EnumerateRegions(IntervalStart, IntervalEnd, Callback))
		{
			return false;
		}
	}

	return true;
}

bool FRegionLane::EnumerateRegions(const double IntervalStart, const double IntervalEnd, TFunctionRef<bool(const FRegion&)> Callback) const
{
	const FInt32Interval OverlapRange = GetElementRangeOverlappingGivenRange<FRegion>(Regions, IntervalStart, IntervalEnd,
		[](const FRegion& Region) { return Region.BeginTime; },
		[](const FRegion& Region) { return Region.EndTime; });

	if (OverlapRange.Min == -1)
	{
		return true;
	}

	for (int32 Index = OverlapRange.Min; Index <= OverlapRange.Max; ++Index)
	{
		if (!Callback(Regions[Index]))
		{
			return false;
		}
	}

	return true;
}

FName GetProviderName()
{
	static const FName Name("MassTraceProvider");
	return Name;
}

void SetupMassTraceAnalysis(TraceServices::IAnalysisSession& InSession)
{
	const TSharedPtr<FMassTraceProvider> MassProvider = MakeShared<FMassTraceProvider>(InSession);
	InSession.AddProvider(GetProviderName(), MassProvider);
	InSession.AddAnalyzer(MakeShared<FMassTraceAnalyzer>(InSession, MassProvider.ToSharedRef()));
}

const IMassTraceProvider& ReadMassTraceProvider(const TraceServices::IAnalysisSession& Session)
{
	return *Session.ReadProvider<IMassTraceProvider>(GetProviderName());
}

IEditableMassTraceProvider& EditMassTraceProvider(TraceServices::IAnalysisSession& Session)
{
	return *Session.EditProvider<IEditableMassTraceProvider>(GetProviderName());
}

} // namespace UE::Mass::Trace

#endif // UE_MASS_TRACE_ANALYSIS_ENABLED
