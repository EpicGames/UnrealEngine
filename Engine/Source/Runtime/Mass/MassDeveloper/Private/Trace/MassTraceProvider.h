// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_MASS_TRACE_ANALYSIS_ENABLED

#include "Common/ProviderLock.h"
#include "Containers/ChunkedArray.h"
#include "Containers/PagedArray.h"
#include "Templates/SharedPointer.h"
#include "Trace/MassTraceTypes.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMassTrace, Display, All)

namespace TraceServices
{
class FAnalysisSessionLock;
class FStringStore;
}

namespace UE::Mass::Trace
{

extern thread_local TraceServices::FProviderLock::FThreadLocalState GMassInsightsProviderLockState;

class FMassTraceProvider
	: public IMassTraceProvider
	, public IEditableMassTraceProvider
{
public:
	explicit FMassTraceProvider(TraceServices::IAnalysisSession& InSession);
	virtual ~FMassTraceProvider() override = default;

	//~ Read operations

	virtual void BeginRead() const override
	{
		Lock.BeginRead(GMassInsightsProviderLockState);
	}
	virtual void EndRead() const override
	{
		Lock.EndRead(GMassInsightsProviderLockState);
	}
	virtual void ReadAccessCheck() const override
	{
		Lock.ReadAccessCheck(GMassInsightsProviderLockState);
	}

	virtual int32 GetFragmentCount() const override;
	virtual const FFragmentInfo* FindFragmentById(uint64 FragmentId) const override;
	virtual const FArchetypeInfo* FindArchetypeById(uint64 ArchetypeId) const override;

	virtual void EnumerateFragments(TFunctionRef<void(const FFragmentInfo& FragmentInfo, int32 Index)> Callback, int32 BeginIndex = 0) const override;

	virtual uint64 GetEntityEventCount() const override;
	virtual TValueOrError<FEntityEventRecord, void> GetEntityEvent(uint64 EventIndex) const override;
	virtual void EnumerateEntityEvents(
		uint64 StartIndex,
		uint64 Count,
		TFunctionRef<void(const FEntityEventRecord&, uint64 /*EventIndex*/)> Callback) const override;

	virtual uint64 GetRegionCount() const override;
	virtual int32 GetLaneCount() const override;

	virtual const FRegionLane* GetLane(int32 Index) const override;

	virtual bool EnumerateRegions(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FRegion&)> Callback) const override;
	virtual void EnumerateLanes(TFunctionRef<void(const FRegionLane&, const int32)> Callback) const override;

	virtual uint64 GetUpdateCounter() const override
	{
		ReadAccessCheck();
		return UpdateCounter;
	}

	//~ Edit operations

	virtual void BeginEdit() const override
	{
		Lock.BeginWrite(GMassInsightsProviderLockState);
	}
	virtual void EndEdit() const override
	{
		Lock.EndWrite(GMassInsightsProviderLockState);
	}
	virtual void EditAccessCheck() const override
	{
		Lock.WriteAccessCheck(GMassInsightsProviderLockState);
	}

	virtual void AddFragment(const FFragmentInfo& FragmentInfo) override;
	virtual void AddArchetype(const FArchetypeInfo& ArchetypeInfo) override;

	virtual void BulkAddEntity(double RecordingTime, double ProfileTime, TConstArrayView<uint64> Entities, TConstArrayView<uint64> ArchetypeIDs) override;
	virtual void BulkMoveEntity(double RecordingTime, double ProfileTime, TConstArrayView<uint64> Entities, TConstArrayView<uint64> ArchetypeIDs) override;
	virtual void BulkDestroyEntity(double RecordingTime, double ProfileTime, TConstArrayView<uint64> Entities) override;

	virtual void AppendRegionBegin(const TCHAR* Name, uint64 ID, double Time) override;
	virtual void AppendRegionBegin(const TCHAR* Name, double Time) override;
	virtual void AppendRegionEnd(const TCHAR* Name, double Time) override;
	virtual void AppendRegionEnd(const uint64 ID, double Time) override;

	virtual void OnAnalysisSessionEnded() override;

private:
	/** Update the depth member of a region to allow overlapping regions to be displayed on separate lanes. */
	int32 CalculateRegionDepth(const FRegion& Item) const;
	void AppendRegionEndInternal(FRegion* OpenRegion, double Time);

private:
	mutable TraceServices::FProviderLock Lock;

	TraceServices::IAnalysisSession& Session;

	TMap<uint64, const FFragmentInfo*> FragmentInfoByID;
	TChunkedArray<FFragmentInfo> FragmentInfos;

	/** Allocation of ranges of fragments, mostly for ArchetypeInfo */
	TChunkedArray<const FFragmentInfo*> FragmentInfoRanges;

	TMap<uint64, const FArchetypeInfo*> ArchetypeByID;
	TChunkedArray<FArchetypeInfo> ArchetypeInfos;

	/** Sorted by Cycle */
	TraceServices::TPagedArray<FEntityEventRecord> EntityEvents;

	/** Open regions inside lanes */
	TMap<FStringView, FRegion*> OpenRegionsByName;
	TMap<uint64, FRegion*> OpenRegionsByID;

	/** Closed regions */
	TArray<FRegionLane> Lanes;

	/** Counter incremented each time region data changes during analysis */
	uint64 UpdateCounter = 0;

	static constexpr uint32 MaxWarningMessages = 100;
	static constexpr uint32 MaxErrorMessages = 100;

	uint32 NumWarnings = 0;
	uint32 NumErrors = 0;
};

} // namespace UE::Mass::Trace

#endif // UE_MASS_TRACE_ANALYSIS_ENABLED
