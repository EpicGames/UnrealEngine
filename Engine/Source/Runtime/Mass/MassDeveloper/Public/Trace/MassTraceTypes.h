// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_MASS_TRACE_ANALYSIS_ENABLED

#include "Common/PagedArray.h" // TraceServices
#include "TraceServices/Model/AnalysisSession.h"

#define UE_API MASSDEVELOPER_API

template <typename ValueType, typename ErrorType> class TValueOrError;

namespace UE::Mass::Trace
{

struct FRegion
{
	double BeginTime = std::numeric_limits<double>::infinity();
	double EndTime = std::numeric_limits<double>::infinity();
	const TCHAR* Text = nullptr;
	/** ID will be zero if the region is identified by Name only */
	uint64 ID = 0;
	int32 Depth = -1;
};

class FRegionLane
{
	friend class FMassTraceProvider;

public:
	explicit FRegionLane(TraceServices::ILinearAllocator& InAllocator) : Regions(InAllocator, 512)
	{
	}

	int32 Num() const
	{
		return static_cast<int32>(Regions.Num());
	}

	/**
	 * Call Callback for every region overlapping the interval defined by IntervalStart and IntervalEnd
 	 * @param IntervalStart the start time of the interval to enumerate regions for.
	 * @param IntervalEnd the end time of the interval to enumerate regions for.
	 * @param Callback a callback called for each region. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	UE_API bool EnumerateRegions(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FRegion&)> Callback) const;

private:
	TraceServices::TPagedArray<FRegion> Regions;
};

enum class EFragmentType : uint8
{
	Unknown,
	Fragment,
	Tag,
	Shared,
	Sparse,
	ConstShared,
	SparseTag
};

struct FFragmentInfo
{
	uint64 Id = INDEX_NONE;
	FString Name;
	uint32 Size = 0;
	EFragmentType Type = EFragmentType::Unknown;

	const TCHAR* GetName() const
	{
		return *Name;
	}
};

struct FArchetypeInfo
{
	uint64 Id = INDEX_NONE;
	TArray<const FFragmentInfo*> Fragments;

	TConstArrayView<const FFragmentInfo*> GetFragments() const
	{
		return MakeArrayView(Fragments);
	}
};

enum class EEntityEventType : uint8
{
	Unknown,
	Created,
	ArchetypeChange,
	Destroyed
};

struct FEntityEventRecord
{
	/** RecordingTime (ElapsedTime) — game time that freezes during pauses.
	 *  This is the time domain the RewindDebugger scrub/view range operates in. */
	double Time = 0;
	/** ProfileTime — wall-clock time that keeps advancing during pauses.
	 *  Used for frame index resolution via IFrameProvider. */
	double ProfileTime = 0;
	uint64 Entity = INDEX_NONE;
	uint64 ArchetypeID = INDEX_NONE;
	EEntityEventType Operation = EEntityEventType::Unknown;
};

class IMassTraceProvider
	: public TraceServices::IProvider
{
public:
	virtual ~IMassTraceProvider() override = default;

	virtual int32 GetFragmentCount() const = 0;
	virtual const FFragmentInfo* FindFragmentById(uint64 FragmentId) const = 0;
	virtual const FArchetypeInfo* FindArchetypeById(uint64 ArchetypeId) const = 0;

	virtual void EnumerateFragments(TFunctionRef<void(const FFragmentInfo& FragmentInfo, int32 Index)> Callback, int32 BeginIndex = 0) const = 0;

	virtual TValueOrError<FEntityEventRecord, void> GetEntityEvent(uint64 EventIndex) const = 0;

	virtual uint64 GetEntityEventCount() const = 0;

	/**
	 * Enumerate up to Count number of events starting at the StartIndex
	 * Enumeration will end early if there is not enough events or if Callback returns false
	 */
	virtual void EnumerateEntityEvents(
		uint64 StartIndex,
		uint64 Count,
		TFunctionRef<void(const FEntityEventRecord&, uint64 EventIndex)> Callback) const = 0;

	/**
	 * @return the amount of currently known regions (including open-ended ones)
	 */
	virtual uint64 GetRegionCount() const = 0;

	/**
	 * @return the number of lanes
	 */
	virtual int32 GetLaneCount() const = 0;

	/**
	 * Direct access to a certain lane at a given index/depth.
	 * The pointer is valid only in the current read scope.
	 * @return a pointer to the lane at the specified depth index or nullptr if Index > GetLaneCount()-1
	 */
	virtual const FRegionLane* GetLane(int32 Index) const = 0;

	/**
	 * Enumerates all regions that overlap a certain time interval. Will enumerate by depth but does not expose lanes.
	 * @param IntervalStart the start time of the interval to enumerate regions for.
	 * @param IntervalEnd the end time of the interval to enumerate regions for.
	 * @param Callback a callback called for each region. Return false to abort iteration.
	 * @returns true if the enumeration finished, false if it was aborted by the callback returning false
	 */
	virtual bool EnumerateRegions(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FRegion&)> Callback) const = 0;

	/**
	 * Will call Callback(Lane, Depth) for each lane in order.
	 */
	virtual void EnumerateLanes(TFunctionRef<void(const FRegionLane&, const int32)> Callback) const = 0;

	/**
	 * @return A monotonically increasing counter that that changes each time new data is added to the provider.
	 * This can be used to detect when to update any (UI-)state dependent on the provider during analysis.
	 */
	virtual uint64 GetUpdateCounter() const = 0;
};

/**
 * The interface to a provider that can consume mutations of region events from a session.
 */
class IEditableMassTraceProvider
	: public TraceServices::IEditableProvider
{
public:
	virtual ~IEditableMassTraceProvider() override = default;

	virtual void AddFragment(const FFragmentInfo& FragmentInfo) = 0;
	virtual void AddArchetype(const FArchetypeInfo& ArchetypeInfo) = 0;

	virtual void BulkAddEntity(double RecordingTime, double ProfileTime, TConstArrayView<uint64> Entities, TConstArrayView<uint64> ArchetypeIDs) = 0;
	virtual void BulkMoveEntity(double RecordingTime, double ProfileTime, TConstArrayView<uint64> Entities, TConstArrayView<uint64> ArchetypeIDs) = 0;
	virtual void BulkDestroyEntity(double RecordingTime, double ProfileTime, TConstArrayView<uint64> Entities) = 0;

	/**
	 * Append a new begin event of a region from the trace session.
	 *
	 * @param Name		The string name of the region.
	 * @param Time		The time in seconds of the begin event of this region.
	 */
	virtual void AppendRegionBegin(const TCHAR* Name, double Time) = 0;

	/**
	 * Append a new begin event of a region from the trace session.
	 *
	 * @param Name		The string name of the region.
	 * @param ID		The ID of the region. Used to uniquely identify regions with the same name.
	 * @param Time		The time in seconds of the begin event of this region.
	 */
	virtual void AppendRegionBegin(const TCHAR* Name, uint64 ID, double Time) = 0;

	/**
	 * Append a new end event of a region from the trace session (by Name).
	 *
	 * @param Name		The string name of the region.
	 * @param Time		The time in seconds of the end event of this region.
	 */
	virtual void AppendRegionEnd(const TCHAR* Name, double Time) = 0;

	/**
	 * Append a new end event of a region from the trace session (by ID).
	 *
	 * @param ID		The ID of the region.
	 * @param Time		The time in seconds of the end event of this region.
	 */
	virtual void AppendRegionEnd(const uint64 ID, double Time) = 0;

	/**
	 * Called from the analyzer once all events have been processed.
	 * Allows postprocessing and error reporting for regions that were never closed.
	 */
	virtual void OnAnalysisSessionEnded() = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

UE_API FName GetProviderName();
UE_API void SetupMassTraceAnalysis(TraceServices::IAnalysisSession& InSession);

UE_API const IMassTraceProvider& ReadMassTraceProvider(const TraceServices::IAnalysisSession& Session);
UE_API IEditableMassTraceProvider& EditMassTraceProvider(TraceServices::IAnalysisSession& Session);
} // namespace UE::Mass::Trace

#undef UE_API

#endif // UE_MASS_TRACE_ANALYSIS_ENABLED
