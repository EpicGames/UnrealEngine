// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassEngine : ModuleRules
	{
		public MassEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			bAllowUETypesInNamespaces = true;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Chaos",
					"Core",
					"CoreUObject",
					"Engine",
					"MassCore",
					"MassEntity",
					"MassSignals",
					"RenderCore",
					"RHI",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new [] {
						"UnrealEd",
					}
				);
			}
		}
	}
}
