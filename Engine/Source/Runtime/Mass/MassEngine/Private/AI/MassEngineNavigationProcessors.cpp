// Copyright Epic Games, Inc. All Rights Reserved.

#include "AI/MassEngineNavigationProcessors.h"

#include "Engine/StaticMesh.h"
#include "Mass/EntityFragments.h"
#include "MassEngineTypes.h"
#include "MassExecutionContext.h"
#include "Mesh/MassEngineMeshFragments.h"
#include "Physics/MassEnginePhysicsFragments.h"
#include "MassSignalSubsystem.h"

#include "AI/NavigationSystemBase.h"
#include "AI/Navigation/NavCollisionBase.h"
#include "AI/Navigation/NavigationElement.h"
#include "AI/Navigation/NavigationRelevantData.h"

// Helper

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEngineNavigationProcessors)

namespace UE::NavigationRelevant
{

bool IsNavigationRelevant(const FMassPhysicsCollisionSettingsFragment& CollisionSettingsFragment,
	const FMassStaticMeshFragment& MeshFragment,
	const FTransformFragment& TransformFragment)
{
	if (TransformFragment.GetTransform().GetScale3D().IsNearlyZero())
	{
		return false;
	}

	// @see UPrimitiveComponent::IsNavigationRelevant()
	{
		if (!CollisionEnabledHasQuery(CollisionSettingsFragment.CollisionType))
		{
			return false;
		}

		const FCollisionResponseContainer& ResponseToChannels = CollisionSettingsFragment.CollisionResponse;
		if (!(ResponseToChannels.GetResponse(ECC_Pawn) == ECR_Block
			|| ResponseToChannels.GetResponse(ECC_Vehicle) == ECR_Block))
		{
			return false;
		}
	}

	// @see UStaticMeshComponent::IsNavigationRelevant()
	const UStaticMesh* Mesh = MeshFragment.Mesh.Get();
	return Mesh != nullptr && !Mesh->IsCompiling() && Mesh->IsNavigationRelevant();
}

void GetNavigationData(const FNavigationElement& Element, const FMassNavigationRelevantParameters& Parameters, const TWeakObjectPtr<UNavCollisionBase> WeakNavCollision, FNavigationRelevantData& OutData)
{
	// @see UPrimitiveComponent::GetNavigationData
	if (Parameters.bFillCollisionUnderneathForNavData)
	{
		FCompositeNavModifier CompositeNavModifier;
		CompositeNavModifier.SetFillCollisionUnderneathForNavmesh(true);
		OutData.Modifiers.Add(CompositeNavModifier);
	}

	// @see UStaticMeshComponent::GetNavigationData
	// but here we don't expose options to override the export (i.e., bOverrideNavigationExport & bForceNavigationObstacle)
	if (UNavCollisionBase* NavCollision = WeakNavCollision.Get())
	{
		if (NavCollision->IsDynamicObstacle())
		{
			NavCollision->GetNavigationModifier(OutData.Modifiers, Element.GetTransform());
		}

		// @todo: allow nav collision to use specific area class
		//if (NavArea)
		//{
		//	NavCollision->GetNavigationModifier(Data.Modifiers, Transform);
		//	TArray<FAreaNavModifier> Areas = Data.Modifiers.GetMutableAreas();
		//	for (FAreaNavModifier& Area : Areas)
		//	{
		//		Area.SetAreaClass(NavArea);
		//	}
		//}
	}
}

bool DoCustomNavigableGeometryExport(const FNavigationElement& Element, const TWeakObjectPtr<UNavCollisionBase> WeakNavCollision, FNavigableGeometryExport& OutGeomExport)
{
	if (const UNavCollisionBase* NavCollision = WeakNavCollision.Get())
	{
		// Dynamic obstacles are handled from GetNavigationData using NavModifiers
		if (NavCollision->IsDynamicObstacle())
		{
			// returning 'false' to skip call to the default export (i.e. ExportRigidBodySetup) since we only use the
			// geometry to mark an obstacle.
			return false;
		}

		if (const bool bHasExportedGeometry = NavCollision->ExportGeometry(Element.GetTransform(), OutGeomExport))
		{
			// returning 'false' to skip call to the default export (i.e. ExportRigidBodySetup) since the export
			// was handled by the NavCollision.
			return false;
		}
	}

	return true;
}

void ConfigureElement(
	const UObject* Owner,
	FNavigationElement& OutElement,
	const FMassStaticMeshFragment& MeshFragment,
	const FTransformFragment& TransformFragment,
	const FMassNavigationRelevantParameters& NavigationRelevantParameters)
{
	const UStaticMesh* StaticMesh = MeshFragment.Mesh.Get();
	const FTransform& Transform = TransformFragment.GetTransform();
	if (StaticMesh)
	{
		OutElement.SetBodySetup(StaticMesh->GetBodySetup());
		OutElement.SetBounds(StaticMesh->GetNavigationBounds(Transform));
	}
	OutElement.SetTransform(Transform);

	OutElement.NavigationDataExportDelegate.BindWeakLambda(Owner,
		[NavigationRelevantParameters, WeakNavCollision = TWeakObjectPtr<UNavCollisionBase>(StaticMesh ? StaticMesh->GetNavCollision() : nullptr)]
		(const FNavigationElement& Element, FNavigationRelevantData& OutNavigationRelevantData)
		{
			GetNavigationData(
				Element,
				NavigationRelevantParameters,
				WeakNavCollision,
				OutNavigationRelevantData);
		});
	OutElement.CustomGeometryExportDelegate.BindWeakLambda(Owner,
		[WeakNavCollision = TWeakObjectPtr<UNavCollisionBase>(StaticMesh ? StaticMesh->GetNavCollision() : nullptr)]
		(const FNavigationElement& Element, FNavigableGeometryExport& OutGeometry, bool& bOutExportDefaultGeometry)
		{
			bOutExportDefaultGeometry = DoCustomNavigableGeometryExport(
				Element,
				WeakNavCollision,
				OutGeometry);
		});
}
} // UE::NavigationRelevant

//----------------------------------------------------------------------//
// UMassNavigationElementProcessor
//----------------------------------------------------------------------//
UMassNavigationElementProcessor::UMassNavigationElementProcessor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, NewRelevantEntityQuery(*this)
{
	bRequiresGameThreadExecution = true;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
}

void UMassNavigationElementProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	NewRelevantEntityQuery.AddConstSharedRequirement<FMassPhysicsCollisionSettingsFragment>();
	NewRelevantEntityQuery.AddConstSharedRequirement<FMassStaticMeshFragment>();
	NewRelevantEntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	NewRelevantEntityQuery.AddTagRequirement<FMassNavigationEvaluatedTag>(EMassFragmentPresence::None);
}

void UMassNavigationElementProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	using namespace UE::NavigationRelevant;

	const UObject* EntityManagerOwner = ExecutionContext.GetEntityManagerChecked().GetOwner();
	check(EntityManagerOwner);

	// @todo: hardcoded for now but should use some default settings and eventually add dedicated component to override settings
	FMassNavigationRelevantParameters Parameters;
	Parameters.bFillCollisionUnderneathForNavData = false;
	const FConstSharedStruct ParametersSharedFragment = EntityManager.GetOrCreateConstSharedFragment(Parameters);

	NewRelevantEntityQuery.ForEachEntityChunk(ExecutionContext,
		[EntityManagerOwner, ParametersSharedFragment](FMassExecutionContext& Context)
		{
			const TConstArrayView<FTransformFragment> TransformFragments = Context.GetFragmentView<FTransformFragment>();
			const FMassStaticMeshFragment& MeshFragment = Context.GetConstSharedFragment<FMassStaticMeshFragment>();
			const FMassPhysicsCollisionSettingsFragment& CollisionSettingsFragment = Context.GetConstSharedFragment<FMassPhysicsCollisionSettingsFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);

				// Both paths below add FMassNavigationEvaluatedTag so this entity is excluded from future runs.
				if (!IsNavigationRelevant(CollisionSettingsFragment, MeshFragment, TransformFragments[EntityIt]))
				{
					Context.Defer().AddTag<FMassNavigationEvaluatedTag>(EntityHandle);

					continue;
				}

				FMassArchetypeSharedFragmentValues SharedValues;
				SharedValues.Add(ParametersSharedFragment);

				FNavigationElement Element(*EntityManagerOwner, EntityHandle.AsNumber());
				ConfigureElement(EntityManagerOwner, Element, MeshFragment, TransformFragments[EntityIt], ParametersSharedFragment.Get<const FMassNavigationRelevantParameters>());
				const FNavigationElementHandle NavigationElementHandle = FNavigationSystem::AddNavigationElement(EntityManagerOwner->GetWorld(), MoveTemp(Element));
				Context.Defer().AddFragmentInstancesWithSharedFragments(EntityHandle, MoveTemp(SharedValues), FMassNavigationRelevantFragment(NavigationElementHandle), FMassNavigationEvaluatedTag{});
			}
		});
}

//----------------------------------------------------------------------//
// UMassDirtyNavigationRelevantUpdateProcessor
//----------------------------------------------------------------------//
UMassDirtyNavigationRelevantUpdateProcessor::UMassDirtyNavigationRelevantUpdateProcessor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bRequiresGameThreadExecution = true;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
}

void UMassDirtyNavigationRelevantUpdateProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	if (const UWorld* World = GetWorld())
	{
		if (UMassSignalSubsystem* SignalSubsystem = World->GetSubsystem<UMassSignalSubsystem>())
		{
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::TransformChanged);
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::MeshChanged);
		}
	}
}

void UMassDirtyNavigationRelevantUpdateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddConstSharedRequirement<FMassPhysicsCollisionSettingsFragment>();
	EntityQuery.AddConstSharedRequirement<FMassStaticMeshFragment>();
	EntityQuery.AddConstSharedRequirement<FMassNavigationRelevantParameters>();
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassNavigationRelevantFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassDirtyNavigationRelevantUpdateProcessor::SignalEntities(
	FMassEntityManager& EntityManager,
	FMassExecutionContext& ExecutionContext,
	FMassSignalNameLookup& EntitySignals)
{
	using namespace UE::NavigationRelevant;

	const UObject* EntityManagerOwner = ExecutionContext.GetEntityManagerChecked().GetOwner();
	check(EntityManagerOwner);

	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[EntityManagerOwner, World = GetWorld()](FMassExecutionContext& Context)
		{
			const TConstArrayView<FTransformFragment> TransformFragments = Context.GetFragmentView<FTransformFragment>();
			const TArrayView<FMassNavigationRelevantFragment> NavigationRelevantFragments = Context.GetMutableFragmentView<FMassNavigationRelevantFragment>();

			const FMassPhysicsCollisionSettingsFragment& CollisionSettingsFragment = Context.GetConstSharedFragment<FMassPhysicsCollisionSettingsFragment>();
			const FMassStaticMeshFragment& MeshFragment = Context.GetConstSharedFragment<FMassStaticMeshFragment>();
			const FMassNavigationRelevantParameters& NavigationRelevantParameters = Context.GetConstSharedFragment<FMassNavigationRelevantParameters>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const bool bWasRelevant = NavigationRelevantFragments[EntityIt].Handle.IsValid();

				if (IsNavigationRelevant(CollisionSettingsFragment, MeshFragment, TransformFragments[EntityIt]))
				{
					FNavigationElement Element(*EntityManagerOwner, Context.GetEntity(EntityIt).AsNumber());
					ConfigureElement(EntityManagerOwner, Element, MeshFragment, TransformFragments[EntityIt], NavigationRelevantParameters);

					if (bWasRelevant)
					{
						FNavigationSystem::UpdateNavigationElement(World, NavigationRelevantFragments[EntityIt].Handle, MoveTemp(Element));
					}
					else
					{
						NavigationRelevantFragments[EntityIt].Handle = FNavigationSystem::AddNavigationElement(World, MoveTemp(Element));
					}
				}
				else if (bWasRelevant)
				{
					FNavigationSystem::RemoveNavigationElement(World, NavigationRelevantFragments[EntityIt].Handle);
				}
			}
		});
}

//----------------------------------------------------------------------//
// UMassNavigationRelevantMassDeinitializer
//----------------------------------------------------------------------//
UMassNavigationRelevantMassDeinitializer::UMassNavigationRelevantMassDeinitializer()
	: EntityQuery(*this)
	, NonRelevantEntityQuery(*this)
{
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
	ObservedTypes.Append({
		FMassStaticMeshFragment::StaticStruct(),
		FTransformFragment::StaticStruct(),
		FMassPhysicsCollisionSettingsFragment::StaticStruct()});
	ObservedOperations = EMassObservedOperationFlags::Remove;
	bRequiresGameThreadExecution = true;
}

void UMassNavigationRelevantMassDeinitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddConstSharedRequirement<FMassStaticMeshFragment>(EMassFragmentPresence::All);
	EntityQuery.AddConstSharedRequirement<FMassPhysicsCollisionSettingsFragment>(EMassFragmentPresence::All);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassNavigationRelevantFragment>(EMassFragmentAccess::ReadOnly);

	NonRelevantEntityQuery.AddTagRequirement<FMassNavigationEvaluatedTag>(EMassFragmentPresence::All);
	NonRelevantEntityQuery.AddRequirement<FMassNavigationRelevantFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
}

void UMassNavigationRelevantMassDeinitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[World = GetWorld()](FMassExecutionContext& Context)
		{
			const TConstArrayView<FMassNavigationRelevantFragment> NavigationRelevantFragments = Context.GetFragmentView<FMassNavigationRelevantFragment>();
			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				if (NavigationRelevantFragments[EntityIt].Handle.IsValid())
				{
					FNavigationSystem::RemoveNavigationElement(World, NavigationRelevantFragments[EntityIt].Handle);
				}
			}

			Context.Defer().RemoveElements<FMassNavigationRelevantFragment, FMassNavigationRelevantParameters, FMassNavigationEvaluatedTag>(Context.GetEntities());
		});

	// Remove the evaluated tag from non-navigation-relevant entities so they can be re-evaluated
	NonRelevantEntityQuery.ForEachEntityChunk(ExecutionContext,
		[](FMassExecutionContext& Context)
		{
			Context.Defer().RemoveElements<FMassNavigationEvaluatedTag>(Context.GetEntities());
		});
}
