// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEngineTypes.h"

DEFINE_LOG_CATEGORY(LogMassEngine);