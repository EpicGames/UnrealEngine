// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDestroyRenderStateContext.h"
#include "SceneProxies/DeferredDecalProxy.h"
#include "PrimitiveSceneProxy.h"
#include "SceneInterface.h"

FMassDestroyRenderStateContext::FMassDestroyRenderStateContext(FSceneInterface* InScene)
	: Scene(InScene)
{
}

FMassDestroyRenderStateContext::~FMassDestroyRenderStateContext()
{
	Process();
}

void FMassDestroyRenderStateContext::Process()
{
	if (!PrimitiveSceneProxies.IsEmpty())
	{
		checkf(Scene, TEXT("Nothing should have been queued up if the scene was not set"));
		Scene->BatchRemovePrimitives(MoveTemp(PrimitiveSceneProxies));
		PrimitiveSceneProxies.Reset();
	}
	if (!DeferredDecalProxies.IsEmpty())
	{
		checkf(Scene, TEXT("Nothing should have been queued up if the scene was not set"));
		TArray<FDeferredDecalUpdateParams> BatchParams;
		BatchParams.Reserve(DeferredDecalProxies.Num());
		for (FDeferredDecalProxy* Proxy : DeferredDecalProxies)
		{
			BatchParams.Add(
				{
					.DecalProxy = Proxy,
					.OperationType = FDeferredDecalUpdateParams::EOperationType::RemoveFromSceneAndDelete
				});
		}
		Scene->BatchUpdateDecals(MoveTemp(BatchParams));
		DeferredDecalProxies.Reset();
	}
}

void FMassDestroyRenderStateContext::DestroyProxy(FMassDestroyRenderStateContext* InContext, FPrimitiveSceneProxy* InPrimitiveSceneProxy, FSceneInterface* InScene)
{
	check(InScene);
	check(InPrimitiveSceneProxy);

	if (InContext)
	{
		checkf(InScene == InContext->Scene, TEXT("Expecting the scene to match the context one"));
		InContext->PrimitiveSceneProxies.Add(InPrimitiveSceneProxy);
	}
	else
	{
		InScene->BatchRemovePrimitives({ InPrimitiveSceneProxy });
	}
}

void FMassDestroyRenderStateContext::DestroyProxy(FMassDestroyRenderStateContext* InContext, FDeferredDecalProxy* InDeferredDecalProxy, FSceneInterface* InScene)
{
	check(InScene);
	check(InDeferredDecalProxy);

	if (InContext)
	{
		checkf(InScene == InContext->Scene, TEXT("Expecting the scene to match the context one"));
		InContext->DeferredDecalProxies.Add(InDeferredDecalProxy);
	}
	else
	{
		TArray<FDeferredDecalUpdateParams> BatchParams;
		BatchParams.Add(
		{
			.DecalProxy = InDeferredDecalProxy,
			.OperationType = FDeferredDecalUpdateParams::EOperationType::RemoveFromSceneAndDelete
		});
		InScene->BatchUpdateDecals(MoveTemp(BatchParams));
	}
}