// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class FSceneInterface;
class FDeferredDecalProxy;
class FPrimitiveSceneProxy;

/*
 * Context to batch the destroy render states.
 */
struct FMassDestroyRenderStateContext
{
public:
	FMassDestroyRenderStateContext(FSceneInterface* InScene);
	~FMassDestroyRenderStateContext();

	bool IsInitialized() const
	{
		return Scene != nullptr;
	}

	void Initialize(FSceneInterface* InScene)
	{
		checkf(!IsInitialized(), TEXT("Context is already initialized"));
		Scene = InScene;
	}

	void Process();

	static void DestroyProxy(FMassDestroyRenderStateContext* InContext, FPrimitiveSceneProxy* InPrimitiveSceneProxy, FSceneInterface* InScene);
	static void DestroyProxy(FMassDestroyRenderStateContext* InContext, FDeferredDecalProxy* InDeferredDecalProxy, FSceneInterface* InScene);

private:
	FSceneInterface* Scene = nullptr;
	TArray<FPrimitiveSceneProxy*> PrimitiveSceneProxies;
	TArray<FDeferredDecalProxy*> DeferredDecalProxies;
};