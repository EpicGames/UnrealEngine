// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/MassEngineMeshFragments.h"

#include "InstancedStaticMeshSceneProxyDesc.h"
#include "Mass/EntityFragments.h"
#include "StaticMeshSceneProxyDesc.h"

struct FTransformFragment;

//----------------------------------------------------------------------//
// FMassStaticMeshFragment
//----------------------------------------------------------------------//
FMassStaticMeshFragment::FMassStaticMeshFragment(TNotNull<const UStaticMesh*> InMesh)
	: Mesh(InMesh)
{
}

FMassRenderStaticMeshFragment::FMassRenderStaticMeshFragment()
	: StaticMeshSceneProxyDesc(MakeShared<FStaticMeshSceneProxyDesc>())
{
}

FMassRenderStaticMeshFragment::FMassRenderStaticMeshFragment(const TSharedRef<FStaticMeshSceneProxyDesc>& InStaticMeshSceneProxyDesc)
	: StaticMeshSceneProxyDesc(InStaticMeshSceneProxyDesc)
{
}

UStaticMesh* FMassRenderStaticMeshFragment::GetStaticMesh()
{
	return StaticMeshSceneProxyDesc ? StaticMeshSceneProxyDesc->StaticMesh : nullptr;
}

const UStaticMesh* FMassRenderStaticMeshFragment::GetStaticMesh() const
{
	return StaticMeshSceneProxyDesc ? StaticMeshSceneProxyDesc->StaticMesh : nullptr;
}

#if WITH_EDITOR
//----------------------------------------------------------------------//
// HTypeElementHandleHitProxy
//----------------------------------------------------------------------//
IMPLEMENT_HIT_PROXY(HTypeElementHandleHitProxy, HHitProxy);

EMouseCursor::Type HTypeElementHandleHitProxy::GetMouseCursor()
{
	return EMouseCursor::Crosshairs;
}
#endif // WITH_EDITOR

//----------------------------------------------------------------------//
// FMassRenderISMFragment
//----------------------------------------------------------------------//
FMassRenderISMFragment::FMassRenderISMFragment()
	: InstancedStaticMeshSceneProxyDesc(MakeShared<FInstancedStaticMeshSceneProxyDesc>())
{
}

FMassRenderISMFragment::FMassRenderISMFragment(const TSharedRef<FInstancedStaticMeshSceneProxyDesc>& InInstancedStaticMeshSceneProxyDesc)
	: InstancedStaticMeshSceneProxyDesc(InInstancedStaticMeshSceneProxyDesc)
{
}

UStaticMesh* FMassRenderISMFragment::GetStaticMesh()
{
	return InstancedStaticMeshSceneProxyDesc ? InstancedStaticMeshSceneProxyDesc->StaticMesh : nullptr;
}

const UStaticMesh* FMassRenderISMFragment::GetStaticMesh() const
{
	return InstancedStaticMeshSceneProxyDesc ? InstancedStaticMeshSceneProxyDesc->StaticMesh : nullptr;
}