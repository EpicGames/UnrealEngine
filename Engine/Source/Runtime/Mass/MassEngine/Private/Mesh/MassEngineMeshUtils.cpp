// Copyright Epic Games, Inc. All Rights Reserved.
#include "Mesh/MassEngineMeshUtils.h"
#include "Mesh/MassEngineMeshFragments.h"
#include "InstancedStaticMeshComponentHelper.h"
#include "InstancedStaticMeshSceneProxyDesc.h"
#include "Mass/EntityFragments.h"
#include "StaticMeshSceneProxyDesc.h"

namespace UE::MassEngine::Mesh
{
	void InitializePrimitiveFragmentsFromComponent(TNotNull<const UActorComponent*> Component, FTransformFragment& TransformFragment, FMassRenderPrimitiveFragment& RenderPrimitiveFragment)
	{
		// Initialize properties not handled by InitializeFromPrimitiveComponent
		const UPrimitiveComponent* PrimitiveComponent = CastChecked<UPrimitiveComponent>(Component);
		TransformFragment.SetTransform(PrimitiveComponent->GetComponentToWorld());
		RenderPrimitiveFragment.bIsVisible = PrimitiveComponent->IsVisible();
		RenderPrimitiveFragment.bRasterizeAsFilledConvexVolume = PrimitiveComponent->bRasterizeAsFilledConvexVolume;
		RenderPrimitiveFragment.CustomPrimitiveData = PrimitiveComponent->GetCustomPrimitiveData();
		RenderPrimitiveFragment.DetailMode = PrimitiveComponent->DetailMode;
		RenderPrimitiveFragment.RuntimeVirtualTextures = PrimitiveComponent->GetRuntimeVirtualTextures();
	}

	void InitializeStaticMeshFragmentsFromComponent(TNotNull<const UActorComponent*> Component, FTransformFragment& TransformFragment, FMassRenderPrimitiveFragment& RenderPrimitiveFragment, FMassRenderStaticMeshFragment& RenderStaticMeshFragment)
	{
		InitializePrimitiveFragmentsFromComponent(Component, TransformFragment, RenderPrimitiveFragment);

		const UStaticMeshComponent* StaticMeshComponent = CastChecked<const UStaticMeshComponent>(Component);

		checkf(RenderStaticMeshFragment.StaticMeshSceneProxyDesc.IsValid(), TEXT("Expecting a valid scene proxy desc"));
		RenderStaticMeshFragment.StaticMeshSceneProxyDesc->InitializeFromStaticMeshComponent(StaticMeshComponent);

		UStaticMesh* StaticMesh = RenderStaticMeshFragment.StaticMeshSceneProxyDesc->StaticMesh;
		checkf(StaticMesh, TEXT("Expecting a valid static mesh at this point"));

		RenderPrimitiveFragment.LocalBounds = StaticMesh->GetBounds();
		RenderPrimitiveFragment.WorldBounds = RenderPrimitiveFragment.LocalBounds.TransformBy(TransformFragment.GetTransform());
	}

	FBoxSphereBounds CalculateInstancedStaticMeshBounds(const FTransformFragment& TransformFragment, const FMassRenderISMFragment& RenderISMFragment, const EBoundsType BoundsType)
	{
		UStaticMesh* StaticMesh = RenderISMFragment.InstancedStaticMeshSceneProxyDesc ? RenderISMFragment.InstancedStaticMeshSceneProxyDesc->StaticMesh : nullptr;
		checkf(StaticMesh, TEXT("Expecting a valid static mesh at this point"));

		if (RenderISMFragment.PerInstanceSMData.Num() > 0)
		{
			const bool bWorldSpace = (BoundsType != EBoundsType::LocalBounds);
			const FBox InstanceBounds = (BoundsType == EBoundsType::NavigationBounds) ? FInstancedStaticMeshComponentHelper::GetInstanceNavigationBounds(RenderISMFragment) : StaticMesh->GetBounds().GetBox();
			const FMatrix ComponentTransformMatrix = TransformFragment.GetTransform().ToMatrixWithScale();
			if (InstanceBounds.IsValid)
			{
				FBoxSphereBounds::Builder BoundsBuilder;
				for (const FInstancedStaticMeshInstanceData& InstanceSMData : RenderISMFragment.PerInstanceSMData)
				{
					if (bWorldSpace)
					{
						BoundsBuilder += InstanceBounds.TransformBy(InstanceSMData.Transform * ComponentTransformMatrix);
					}
					else
					{
						BoundsBuilder += InstanceBounds.TransformBy(InstanceSMData.Transform);
					}
				}
				return BoundsBuilder;
			}
		}
		return FBoxSphereBounds(FVector::ZeroVector, FVector::ZeroVector, 0.f);
	}

	void InitializeISMFragmentsFromComponent(TNotNull<const UActorComponent*> Component, FTransformFragment& TransformFragment, FMassRenderPrimitiveFragment& RenderPrimitiveFragment, FMassRenderISMFragment& RenderISMFragment)
	{
		InitializePrimitiveFragmentsFromComponent(Component, TransformFragment, RenderPrimitiveFragment);

		const UInstancedStaticMeshComponent* ISMComponent = CastChecked<const UInstancedStaticMeshComponent>(Component);

		checkf(RenderISMFragment.InstancedStaticMeshSceneProxyDesc.IsValid(), TEXT("Expecting a valid scene proxy desc"));
		RenderISMFragment.InstancedStaticMeshSceneProxyDesc->InitializeFromStaticMeshComponent(ISMComponent);

		RenderISMFragment.InstancedStaticMeshSceneProxyDesc->bCollisionEnabled = !!RenderISMFragment.InstancedStaticMeshSceneProxyDesc->bCollisionEnabled && !ISMComponent->bDisableCollision;
		RenderISMFragment.AdditionalRandomSeeds = ISMComponent->AdditionalRandomSeeds;
		// @todo optimize the TArray to TSparseArray copy here
		RenderISMFragment.PerInstanceSMData.Reset();
		RenderISMFragment.PerInstanceSMData.Reserve(ISMComponent->PerInstanceSMData.Num());
		for (const FInstancedStaticMeshInstanceData& Data : ISMComponent->PerInstanceSMData)
		{
			RenderISMFragment.PerInstanceSMData.Add(Data);
		}
		RenderISMFragment.PerInstanceSMCustomData = ISMComponent->PerInstanceSMCustomData;
		RenderISMFragment.InstancingRandomSeed = ISMComponent->InstancingRandomSeed;

		RenderPrimitiveFragment.LocalBounds = CalculateInstancedStaticMeshBounds(TransformFragment, RenderISMFragment, EBoundsType::LocalBounds);
		RenderPrimitiveFragment.WorldBounds = CalculateInstancedStaticMeshBounds(TransformFragment, RenderISMFragment, EBoundsType::WorldBounds);
	}


#if WITH_EDITOR
	void InitializePrimitiveSceneProxyDescFromEditorFragment(const FMassEditorVisualizationMeshFragment& EditorMeshFragment, FPrimitiveSceneProxyDesc& PrimitiveSceneProxyDesc)
	{
		PrimitiveSceneProxyDesc.bShouldRenderSelected = EditorMeshFragment.bShouldRenderSelected;
		PrimitiveSceneProxyDesc.bSelected = EditorMeshFragment.bShouldRenderSelected;
		PrimitiveSceneProxyDesc.bIndividuallySelected = false;
		PrimitiveSceneProxyDesc.bIsHiddenEd = EditorMeshFragment.bHiddenInEditor;
		PrimitiveSceneProxyDesc.bIsVisibleEditor = !EditorMeshFragment.bHiddenInEditor;
		PrimitiveSceneProxyDesc.bLevelInstanceEditingState = EditorMeshFragment.bLevelInstanceEditingState;

	}
#endif // WITH_EDITOR

	void InitializeStaticMeshSceneProxyDescFromFragment(const FMassStaticMeshFragment& StaticMeshFragment,	const FMassVisualizationMeshFragment& MeshFragment, const FTransformFragment& TransformFragment, FStaticMeshSceneProxyDesc& StaticMeshSceneProxyDesc)
	{
		StaticMeshSceneProxyDesc.StaticMesh = const_cast<UStaticMesh*>(StaticMeshFragment.Mesh.Get());

		//////////////////////////////////////////////////////////////////////////////	
		// This should maybe be done by the RenderStateHelpers
		if (FStaticMeshRenderData* RenderData = StaticMeshSceneProxyDesc.StaticMesh ? StaticMeshSceneProxyDesc.StaticMesh->GetRenderData() : nullptr)
		{
			StaticMeshSceneProxyDesc.NaniteResources = RenderData->NaniteResourcesPtr.Get();
		}
#if WITH_EDITOR
		StaticMeshSceneProxyDesc.TextureStreamingTransformScale = TransformFragment.GetTransform().GetMaximumAxisScale();
#endif
		// This should maybe be done by the RenderStateHelpers
		//////////////////////////////////////////////////////////////////////////////	

		StaticMeshSceneProxyDesc.TranslucencySortPriority = MeshFragment.TranslucencySortPriority;
		StaticMeshSceneProxyDesc.CastShadow = MeshFragment.CastShadow;
		StaticMeshSceneProxyDesc.bCastShadowAsTwoSided = MeshFragment.CastShadowAsTwoSided;
		StaticMeshSceneProxyDesc.bCastHiddenShadow = MeshFragment.CastHiddenShadow;
		StaticMeshSceneProxyDesc.bCastFarShadow = MeshFragment.CastFarShadow;
		StaticMeshSceneProxyDesc.bCastInsetShadow = MeshFragment.CastInsetShadow;
		StaticMeshSceneProxyDesc.bCastContactShadow = MeshFragment.CastContactShadow;
		StaticMeshSceneProxyDesc.bAffectDynamicIndirectLighting = MeshFragment.AffectDynamicIndirectLighting;
		StaticMeshSceneProxyDesc.bAffectIndirectLightingWhileHidden = MeshFragment.AffectIndirectLightingWhileHidden;
		StaticMeshSceneProxyDesc.bReceivesDecals = MeshFragment.ReceivesDecals;
		StaticMeshSceneProxyDesc.bRenderCustomDepth = MeshFragment.RenderCustomDepth;
		StaticMeshSceneProxyDesc.CustomDepthStencilValue = (int32)MeshFragment.CustomDepthStencilValue;
		StaticMeshSceneProxyDesc.LightingChannels = MeshFragment.LightingChannels;

		if (MeshFragment.NeverDistanceCull)
		{
			StaticMeshSceneProxyDesc.CachedMaxDrawDistance = 0.0f;
		}
	}

	void InitializeInstanceStaticMeshSceneProxyDescFromFragment(const FMassStaticMeshFragment& StaticMeshFragment, const FMassVisualizationMeshFragment& MeshFragment, const FTransformFragment& TransformFragment, FInstancedStaticMeshSceneProxyDesc& ISMSceneProxyDesc)
	{
		InitializeStaticMeshSceneProxyDescFromFragment(StaticMeshFragment, MeshFragment, TransformFragment, ISMSceneProxyDesc);
	};

}