// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/MassEngineRenderISMProcessors.h"

#include "MassDestroyRenderStateContext.h"
#include "MassEngineTypes.h"
#include "Mass/EntityFragments.h"
#include "MassExecutionContext.h"
#include "MassISMRenderStateHelper.h"
#include "MassRenderStateHelper.h"
#include "MassSignalSubsystem.h"
#include "MassStaticMeshRenderStateHelper.h"
#include "Mesh/MassEngineMeshFragments.h"
#include "Mesh/MassEngineMeshUtils.h"
#include "Misc/HashBuilder.h"
#include "StaticMeshComponentHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEngineRenderISMProcessors)

//----------------------------------------------------------------------//
// UMassInstantiateStaticMeshAndCreateISMRenderStateProcessor
//----------------------------------------------------------------------//
void UMassInstantiateStaticMeshAndCreateISMRenderStateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);
 	EntityQuery.AddRequirement<FMassTypeElementFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery.AddTagRequirement<FMassRenderISMCandidateTag>(EMassFragmentPresence::All);
	EntityQuery.AddConstSharedRequirement<FMassRenderISMLinkFragment>(EMassFragmentPresence::None);
	EntityQuery.AddRequirement<FMassRenderISMInstanceFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
	EntityQuery.AddConstSharedRequirement<FMassISMGroupingFragment>(EMassFragmentPresence::Optional);

	ISMEntityComposition.Add<FTransformFragment>();
	ISMEntityComposition.Add<FMassVisualizationMeshFragment>();
	ISMEntityComposition.Add<FMassRenderStateFragment>();
	ISMEntityComposition.Add<FMassRenderPrimitiveFragment>();
	ISMEntityComposition.Add<FMassRenderISMFragment>();
	ISMEntityArchetype = EntityManager->CreateArchetype(ISMEntityComposition, FMassArchetypeCreationParams{"RenderISM"});
	ProcessorEntityCreationRequirements.AddCreatedArchetype(ISMEntityComposition);

	ISMEntityCompositionWithMaterialOverrides = ISMEntityComposition;
	ISMEntityCompositionWithMaterialOverrides.Add<FMassOverrideMaterialsFragment>();
	ISMEntityArchetypeWithMaterialOverrides = EntityManager->CreateArchetype(ISMEntityCompositionWithMaterialOverrides, FMassArchetypeCreationParams{"RenderISMwMaterialOverrides"});
	ProcessorEntityCreationRequirements.AddCreatedArchetype(ISMEntityCompositionWithMaterialOverrides);

#if WITH_EDITOR
	ISMEntityCompositionWithEditorMesh = ISMEntityComposition;
	ISMEntityCompositionWithEditorMesh.Add<FMassEditorVisualizationMeshFragment>();
	ISMEntityArchetypeWithEditorMesh = EntityManager->CreateArchetype(ISMEntityCompositionWithEditorMesh, FMassArchetypeCreationParams{"RenderISMwEditorMesh"});
	ProcessorEntityCreationRequirements.AddCreatedArchetype(ISMEntityCompositionWithEditorMesh);

	ISMEntityCompositionWithMaterialOverridesAndEditorMesh = ISMEntityCompositionWithMaterialOverrides;
	ISMEntityCompositionWithMaterialOverridesAndEditorMesh.Add<FMassEditorVisualizationMeshFragment>();
	ISMEntityArchetypeWithMaterialOverridesAndEditorMesh = EntityManager->CreateArchetype(ISMEntityCompositionWithMaterialOverridesAndEditorMesh, FMassArchetypeCreationParams{"RenderISMwMaterialOverridesAndEditorMesh"});
	ProcessorEntityCreationRequirements.AddCreatedArchetype(ISMEntityCompositionWithMaterialOverridesAndEditorMesh);
#endif // WITH_EDITOR

}

void UMassInstantiateStaticMeshAndCreateISMRenderStateProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	struct FISMEntityInfo
	{
		uint32 SourceSharedFragmentHash = 0;
		const FMassStaticMeshFragment& StaticMeshFragment;
		const FMassVisualizationMeshFragment& MeshFragment;
		uint32 InstanceCount = 0;
		uint32 CurrentInstance = 0;
	};
	struct FISMEntityBatch
	{
		const FMassOverrideMaterialsFragment* OverrideMaterialsFragment = nullptr;
		const FMassEditorVisualizationMeshFragment* EditorMeshFragment = nullptr;
		TArray<FISMEntityInfo> EntitiesInfo;
		TArray<FMassEntityHandle> CreatedEntities;
	};

	TArray<FISMEntityBatch> ISMEntityBatches;

	auto RetrieveISMEntityInfo = [&ISMEntityBatches](
		const FMassStaticMeshFragment& StaticMeshFragment,
		const FMassVisualizationMeshFragment& MeshFragment,
		const FMassOverrideMaterialsFragment* OverrideMaterialsFragment,
		const FMassEditorVisualizationMeshFragment* EditorMeshFragment,
		const FMassISMGroupingFragment* ISMGrouping,
		bool bCreateIfNotFound,
		FMassEntityHandle* ISMEntityHandle = nullptr) -> FISMEntityInfo*
	{
		FISMEntityBatch* Batch = ISMEntityBatches.FindByPredicate(
		[OverrideMaterialsFragment, EditorMeshFragment](const FISMEntityBatch& Batch)
			{
				return Batch.OverrideMaterialsFragment == OverrideMaterialsFragment &&
					Batch.EditorMeshFragment == EditorMeshFragment;
			});
		if (!Batch)
		{
			if (!bCreateIfNotFound)
			{
				return nullptr;
			}
			Batch = &ISMEntityBatches.Emplace_GetRef(OverrideMaterialsFragment, EditorMeshFragment);
		}

		// Build the source shared fragment hash.
		// Include ISM grouping ID when present so entities with different group scoping
		// (e.g., per-cell) get separate ISM entities even with the same mesh/vis flags.
		FHashBuilder SourceSharedFragmentsHash;
		SourceSharedFragmentsHash << &StaticMeshFragment << &MeshFragment;
		if (ISMGrouping)
		{
			SourceSharedFragmentsHash << ISMGrouping->GroupId;
		}

		// Find if the ISM entity already exist, if not add it
		FISMEntityInfo* ISMEntity;
		int32 Index = Batch->EntitiesInfo.IndexOfByPredicate([Hash = SourceSharedFragmentsHash.GetHash()](const FISMEntityInfo& ISMEntityInfo) { return ISMEntityInfo.SourceSharedFragmentHash == Hash; });
		if (Index == INDEX_NONE)
		{
			if (!bCreateIfNotFound)
			{
				return nullptr;
			}
			Index = Batch->EntitiesInfo.Num();
			ISMEntity = &Batch->EntitiesInfo.Emplace_GetRef(SourceSharedFragmentsHash.GetHash(), StaticMeshFragment, MeshFragment);
		}
		else
		{
			ISMEntity = &Batch->EntitiesInfo[Index];
		}

		if (ISMEntityHandle)
		{
			checkf(Index < Batch->CreatedEntities.Num(), TEXT("Expecting all the entities to be created at this point"));
			*ISMEntityHandle = Batch->CreatedEntities[Index];
		}
		return ISMEntity;
	};

	//////////////////////////////////////////////////////////////////////////////
	// First gather all the different ISM entities we need to create.
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&RetrieveISMEntityInfo](FMassExecutionContext& Context)
		{
			const FMassStaticMeshFragment& StaticMeshFragment = Context.GetConstSharedFragment<FMassStaticMeshFragment>();
			const FMassVisualizationMeshFragment& MeshFragment = Context.GetConstSharedFragment<FMassVisualizationMeshFragment>();
			const FMassOverrideMaterialsFragment* OverrideMaterialsFragment = Context.GetConstSharedFragmentPtr<FMassOverrideMaterialsFragment>();
#if WITH_EDITOR
			const FMassEditorVisualizationMeshFragment* EditorMeshFragment = Context.GetConstSharedFragmentPtr<FMassEditorVisualizationMeshFragment>();
#else
			const FMassEditorVisualizationMeshFragment* EditorMeshFragment = nullptr;
#endif // WITH_EDITOR

			// Find the batch matching the shared fragment
			const FMassISMGroupingFragment* ISMGrouping = Context.GetConstSharedFragmentPtr<FMassISMGroupingFragment>();
			FISMEntityInfo* ISMEntityInfo = RetrieveISMEntityInfo(StaticMeshFragment, MeshFragment, OverrideMaterialsFragment, EditorMeshFragment, ISMGrouping, /*bCreateIfNotFound*/true);
			checkf(ISMEntityInfo, TEXT("Expecting the ISM to be created"));
			ISMEntityInfo->InstanceCount += Context.GetNumEntities();
		});

	//////////////////////////////////////////////////////////////////////////////
	// Batch create the mass entities
	for (FISMEntityBatch& Batch : ISMEntityBatches)
	{
		FMassArchetypeHandle ArchetypeHandle;
		FMassArchetypeSharedFragmentValues SharedFragmentValues;
		if (!Batch.OverrideMaterialsFragment && !Batch.EditorMeshFragment)
		{
			ArchetypeHandle = ISMEntityArchetype;
		}
		else if (Batch.OverrideMaterialsFragment && !Batch.EditorMeshFragment)
		{
			SharedFragmentValues.Add(FConstSharedStruct::Make(*Batch.OverrideMaterialsFragment));
			ArchetypeHandle = ISMEntityArchetypeWithMaterialOverrides;
		}
#if WITH_EDITOR
		else if (!Batch.OverrideMaterialsFragment && Batch.EditorMeshFragment)
		{
			SharedFragmentValues.Add(FConstSharedStruct::Make(*Batch.EditorMeshFragment));
			ArchetypeHandle = ISMEntityArchetypeWithEditorMesh;
		}
		else if (Batch.OverrideMaterialsFragment && Batch.EditorMeshFragment)
		{
			SharedFragmentValues.Add(FConstSharedStruct::Make(*Batch.OverrideMaterialsFragment));
			SharedFragmentValues.Add(FConstSharedStruct::Make(*Batch.EditorMeshFragment));
			ArchetypeHandle = ISMEntityArchetypeWithMaterialOverridesAndEditorMesh;
		}
#endif // WITH_EDITOR

		for (const FISMEntityInfo& EntityInfo : Batch.EntitiesInfo)
		{
			FMassArchetypeSharedFragmentValues EntitySharedFragmentValues(SharedFragmentValues);
			EntitySharedFragmentValues.Add(FConstSharedStruct::Make(EntityInfo.MeshFragment));
			EntitySharedFragmentValues.Sort();
			Batch.CreatedEntities.Add(ExecutionContext.CreateEntity(ArchetypeHandle, EntitySharedFragmentValues));
		}
	}

	//////////////////////////////////////////////////////////////////////////////
	// Gather all the transforms for each ISM entity and linked them to the created ISM entity
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&EntityManager, &RetrieveISMEntityInfo](FMassExecutionContext& Context)
		{
			const FMassStaticMeshFragment& StaticMeshFragment = Context.GetConstSharedFragment<FMassStaticMeshFragment>();
			const FMassVisualizationMeshFragment& MeshFragment = Context.GetConstSharedFragment<FMassVisualizationMeshFragment>();
			const FMassOverrideMaterialsFragment* OverrideMaterialsFragment = Context.GetConstSharedFragmentPtr<FMassOverrideMaterialsFragment>();
			const TConstArrayView<FTransformFragment> TransformFragments = Context.GetFragmentView<FTransformFragment>();
			const TConstArrayView<FMassTypeElementFragment> TypeElementFragments = Context.GetFragmentView<FMassTypeElementFragment>();
#if WITH_EDITOR
			const FMassEditorVisualizationMeshFragment* EditorMeshFragment = Context.GetConstSharedFragmentPtr<FMassEditorVisualizationMeshFragment>();
#else
			const FMassEditorVisualizationMeshFragment* EditorMeshFragment = nullptr;
#endif // WITH_EDITOR

			FMassEntityHandle ISMEntityHandle;
			const FMassISMGroupingFragment* ISMGrouping = Context.GetConstSharedFragmentPtr<FMassISMGroupingFragment>();
			FISMEntityInfo* ISMEntityInfo = RetrieveISMEntityInfo(StaticMeshFragment, MeshFragment, OverrideMaterialsFragment, EditorMeshFragment, ISMGrouping, /*bCreateIfNotFound*/false, &ISMEntityHandle);
			checkf(ISMEntityInfo, TEXT("Expecting the ISM to be created"));
			FMassEntityView ISMEntityView(EntityManager, ISMEntityHandle);

			FMassRenderISMLinkFragment LinkedISMEntityFragmentValues;
			LinkedISMEntityFragmentValues.LinkedEntityHandle = ISMEntityHandle;
			const FConstSharedStruct LinkedISMEntityFragment = EntityManager.GetOrCreateConstSharedFragment(LinkedISMEntityFragmentValues);

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FMassRenderISMFragment& ISMDescFragment = ISMEntityView.GetFragmentData<FMassRenderISMFragment>();
				if (ISMDescFragment.PerInstanceSMData.Num() == 0)
				{
					ISMDescFragment.PerInstanceSMData.Reset();
					ISMDescFragment.PerInstanceSMData.Reserve(ISMEntityInfo->InstanceCount);
#if WITH_EDITOR
					ISMDescFragment.PerInstanceHitProxy.Reset();
					ISMDescFragment.PerInstanceHitProxy.Reserve(ISMEntityInfo->InstanceCount);
#endif // WITH_EDITOR
				}
				FMassRenderISMInstanceFragment ISMInfoFragment;
				ISMInfoFragment.InstanceIndex = ISMDescFragment.PerInstanceSMData.Add(TransformFragments[*EntityIt].GetTransform().ToMatrixWithScale());
#if WITH_EDITOR
				ISMInfoFragment.HitProxyIndex = ISMDescFragment.PerInstanceHitProxy.Add(new HTypeElementHandleHitProxy(TypeElementFragments.Num()? TypeElementFragments[*EntityIt].TypeElementHandle : FTypedElementHandle{}));
#endif // WITH_EDITOR

				const FMassEntityHandle EntityHandle = Context.GetEntity(*EntityIt);
				FMassArchetypeSharedFragmentValues SharedValues;
				SharedValues.Add(LinkedISMEntityFragment);
				Context.Defer().AddFragmentInstancesWithSharedFragments(EntityHandle, MoveTemp(SharedValues), MoveTemp(ISMInfoFragment));

				ISMEntityInfo->CurrentInstance++;
			}
		});

	//////////////////////////////////////////////////////////////////////////////
	// Finished the creation of the ISM entities
	for (const FISMEntityBatch& Batch : ISMEntityBatches)
	{
		for (int32 i = 0; i < Batch.EntitiesInfo.Num(); ++i)
		{
			const FISMEntityInfo& ISMInfo = Batch.EntitiesInfo[i];
			const FMassEntityHandle ISMEntityHandle = Batch.CreatedEntities[i];
			const FMassEntityView ISMEntityView(EntityManager, ISMEntityHandle);

			FTransformFragment& TransformFragment = ISMEntityView.GetFragmentData<FTransformFragment>();
			FMassRenderPrimitiveFragment& RenderPrimitiveFragment = ISMEntityView.GetFragmentData<FMassRenderPrimitiveFragment>();
			FMassRenderISMFragment& ISMDescFragment = ISMEntityView.GetFragmentData<FMassRenderISMFragment>();
			FMassRenderStateFragment& RenderStateFragment = ISMEntityView.GetFragmentData<FMassRenderStateFragment>();


			checkf(ISMDescFragment.InstancedStaticMeshSceneProxyDesc, TEXT("Expecting a valid proxy desc"));
			UE::MassEngine::Mesh::InitializeInstanceStaticMeshSceneProxyDescFromFragment(ISMInfo.StaticMeshFragment, ISMInfo.MeshFragment, TransformFragment, *ISMDescFragment.InstancedStaticMeshSceneProxyDesc);
#if WITH_EDITOR
			if (Batch.EditorMeshFragment)
			{
				UE::MassEngine::Mesh::InitializePrimitiveSceneProxyDescFromEditorFragment(*Batch.EditorMeshFragment, *ISMDescFragment.InstancedStaticMeshSceneProxyDesc);
			}
#endif // WITH_EDITOR

			RenderPrimitiveFragment.bIsVisible = true;
			RenderPrimitiveFragment.LocalBounds = UE::MassEngine::Mesh::CalculateInstancedStaticMeshBounds(TransformFragment, ISMDescFragment, UE::MassEngine::Mesh::EBoundsType::LocalBounds);
			RenderPrimitiveFragment.WorldBounds = UE::MassEngine::Mesh::CalculateInstancedStaticMeshBounds(TransformFragment, ISMDescFragment, UE::MassEngine::Mesh::EBoundsType::WorldBounds);

			RenderStateFragment.CreateRenderStateHelper<FMassISMRenderStateHelper>(ISMEntityHandle, &EntityManager, RenderPrimitiveFragment, Batch.OverrideMaterialsFragment, ISMDescFragment);
		}
	}
}

//----------------------------------------------------------------------//
// UMassDirtyRenderISMInstanceProcessor
//----------------------------------------------------------------------//
UMassDirtyRenderISMInstanceProcessor::UMassDirtyRenderISMInstanceProcessor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SetupRenderState;
}

void UMassDirtyRenderISMInstanceProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	if (const UWorld* World = GetWorld())
	{
		if (UMassSignalSubsystem* SignalSubsystem = World->GetSubsystem<UMassSignalSubsystem>())
		{
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::TransformChanged);
#if WITH_EDITOR
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::SelectionChanged);
#endif // WITH_EDITOR
		}
	}
}

void UMassDirtyRenderISMInstanceProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRenderISMInstanceFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddConstSharedRequirement<FMassRenderISMLinkFragment>();
	EntityQuery.AddLinkedEntityRequirement<FMassRenderISMFragment, FMassRenderISMLinkFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddLinkedEntityRequirement<FMassRenderStateFragment, FMassRenderISMLinkFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassDirtyRenderISMInstanceProcessor::SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext, FMassSignalNameLookup& EntitySignals)
{
	const UObject* EntityManagerOwner = ExecutionContext.GetEntityManagerChecked().GetOwner();
	check(EntityManagerOwner);

	TArray<FName> SignalsForEntity;
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&EntitySignals, &SignalsForEntity](FMassExecutionContext& Context)
		{
			const TConstArrayView<FMassRenderISMInstanceFragment> ISMInfoFragments = Context.GetFragmentView<FMassRenderISMInstanceFragment>();
			const FMassRenderISMLinkFragment LinkedISMEntityFragment = Context.GetConstSharedFragment<FMassRenderISMLinkFragment>();
			FMassRenderISMFragment* ISMDescFragment = Context.GetIndirectFragmentPtr<FMassRenderISMFragment>(LinkedISMEntityFragment.LinkedEntityHandle);
			checkf(ISMDescFragment, TEXT("Expecting a ISM desc fragment"));
			FMassRenderStateFragment* RenderStateFragment = Context.GetIndirectFragmentPtr<FMassRenderStateFragment>(LinkedISMEntityFragment.LinkedEntityHandle);
			checkf(RenderStateFragment, TEXT("Expecting a render state fragment"));
			const TConstArrayView<FTransformFragment> TransformFragments = Context.GetFragmentView<FTransformFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const int32 InstanceIndex = ISMInfoFragments[*EntityIt].InstanceIndex;
				checkf(ISMDescFragment->PerInstanceSMData.IsValidIndex(InstanceIndex), TEXT("The entity is out of sync with its instance"));
#if WITH_EDITOR
				const int32 HitProxyIndex = ISMInfoFragments[*EntityIt].HitProxyIndex;
				checkf(ISMDescFragment->PerInstanceHitProxy.IsValidIndex(HitProxyIndex), TEXT("The entity is out of sync with its instance type element"));
				checkf(ISMDescFragment->PerInstanceSMData.Num() == ISMDescFragment->PerInstanceHitProxy.Num(), TEXT("The ISM entity is out of sync"));
#endif // WITH_EDITOR

				SignalsForEntity.Reset();
				EntitySignals.GetSignalsForEntity(Context.GetEntity(*EntityIt), SignalsForEntity);

				const bool bTransformChanged = SignalsForEntity.Contains(UE::Mass::Signals::TransformChanged);
				if (SignalsForEntity.Num() == 1 && bTransformChanged)
				{
					ISMDescFragment->PerInstanceSMData[InstanceIndex] = TransformFragments[*EntityIt].GetTransform().ToMatrixWithScale();
					// @TODO optimize this to use updatable ISM Scene proxy, but for now Trigger an ISM recreation
					RenderStateFragment->GetRenderStateHelper().DestroyRenderState(nullptr);
					Context.Defer().RemoveTag<FMassSceneProxyCreatedTag>(LinkedISMEntityFragment.LinkedEntityHandle);
				}
				else
				{
					Context.Defer().RemoveElements<FMassRenderISMInstanceFragment, FMassRenderISMLinkFragment>(Context.GetEntity(*EntityIt));

					// Delete the ism entity if this is the last instance in the desc
					if (ISMDescFragment->PerInstanceSMData.Num() == 1)
					{
						Context.Defer().DestroyEntity(LinkedISMEntityFragment.LinkedEntityHandle);
					}
					else
					{
						// Remove the transform of that instance and trigger a SetupRenderState
						ISMDescFragment->PerInstanceSMData.RemoveAt(InstanceIndex);
#if WITH_EDITOR
						ISMDescFragment->PerInstanceHitProxy.RemoveAt(HitProxyIndex);
#endif // WITH_EDITOR
						// @TODO optimize this to use updatable ISM Scene proxy, but for now Trigger an ISM recreation
						RenderStateFragment->GetRenderStateHelper().DestroyRenderState(nullptr);
						Context.Defer().RemoveTag<FMassSceneProxyCreatedTag>(LinkedISMEntityFragment.LinkedEntityHandle);
					}
				}
			}
		});
}

// ----------------------------------------------------------------------//
// UMassISMDestroyRenderStateProcessor
//----------------------------------------------------------------------//
UMassISMDestroyRenderStateProcessor::UMassISMDestroyRenderStateProcessor()
{
	ObservedTypes.Add(FMassRenderISMInstanceFragment::StaticStruct());
	ObservedTypes.Add(FMassRenderISMLinkFragment::StaticStruct());
	ObservedTypes.Add(FMassRenderISMCandidateTag::StaticStruct());
}

void UMassISMDestroyRenderStateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);

	EntityQuery.AddRequirement<FMassRenderISMInstanceFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddConstSharedRequirement<FMassRenderISMLinkFragment>();
	EntityQuery.AddLinkedEntityRequirement<FMassRenderISMFragment, FMassRenderISMLinkFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddLinkedEntityRequirement<FMassRenderStateFragment, FMassRenderISMLinkFragment>(EMassFragmentAccess::ReadOnly);
}

void UMassISMDestroyRenderStateProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	FMassDestroyRenderStateContext BatchingContext(nullptr);

	TArray<FMassEntityHandle> EntitiesToDestroy;

	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&BatchingContext, &EntitiesToDestroy](FMassExecutionContext& Context)
		{
			const TArrayView<FMassRenderISMInstanceFragment> ISMInfoFragments = Context.GetMutableFragmentView<FMassRenderISMInstanceFragment>();
			const FMassRenderISMLinkFragment LinkedISMEntityFragment = Context.GetConstSharedFragment<FMassRenderISMLinkFragment>();
			FMassRenderISMFragment* ISMDescFragment = Context.GetIndirectFragmentPtr<FMassRenderISMFragment>(LinkedISMEntityFragment.LinkedEntityHandle);
			checkf(ISMDescFragment, TEXT("Expecting a ISM desc fragment"));
			FMassRenderStateFragment* RenderStateFragment = Context.GetIndirectFragmentPtr<FMassRenderStateFragment>(LinkedISMEntityFragment.LinkedEntityHandle);
			checkf(RenderStateFragment, TEXT("Expecting a render state fragment"));

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const int32 InstanceIndex = ISMInfoFragments[*EntityIt].InstanceIndex;

				// @Todo: Temps fix for a bug in Mass that prevents us from adding right requirements which would prevent multiple observer call
				// Check description of the problem here: UMassBaseStaticMeshDestroyRenderStateProcessor::ConfigureQueries
				if (InstanceIndex == INDEX_NONE)
				{
					continue;
				}
				checkf(ISMDescFragment->PerInstanceSMData.IsValidIndex(InstanceIndex), TEXT("The entity is out of sync with its instance"));
#if WITH_EDITOR
				const int32 HitProxyIndex = ISMInfoFragments[*EntityIt].HitProxyIndex;
				checkf(ISMDescFragment->PerInstanceHitProxy.IsValidIndex(HitProxyIndex), TEXT("The entity is out of sync with its instance type element"));
				checkf(ISMDescFragment->PerInstanceSMData.Num() == ISMDescFragment->PerInstanceHitProxy.Num(), TEXT("The ISM entity is out of sync"))
#endif // WITH_EDITOR

				// Whether we remove one instance or it is the last one, let destroy the render state.
				if (!BatchingContext.IsInitialized())
				{
					BatchingContext.Initialize(RenderStateFragment->GetRenderStateHelper().GetScene());
				}
				RenderStateFragment->GetRenderStateHelper().DestroyRenderState(&BatchingContext);

				// Delete the ism entity if this is the last instance in the desc
				if (ISMDescFragment->PerInstanceSMData.Num() == 1)
				{
					EntitiesToDestroy.Add(LinkedISMEntityFragment.LinkedEntityHandle);
				}
				else
				{
					// Remove the transform of that instance and trigger a CreateRenderState
					ISMDescFragment->PerInstanceSMData.RemoveAt(InstanceIndex);
#if WITH_EDITOR
					ISMDescFragment->PerInstanceHitProxy.RemoveAt(HitProxyIndex);
#endif // WITH_EDITOR
					Context.Defer().RemoveTag<FMassSceneProxyCreatedTag>(LinkedISMEntityFragment.LinkedEntityHandle);
				}

				FMassEntityHandle EntityHandle = Context.GetEntity(*EntityIt);

				ISMInfoFragments[*EntityIt].InstanceIndex = INDEX_NONE;
				Context.Defer().RemoveElements<FMassRenderISMInstanceFragment, FMassRenderISMLinkFragment>(EntityHandle);
			}
		});

	if (BatchingContext.IsInitialized())
	{
		BatchingContext.Process();
	}

	if (EntitiesToDestroy.Num())
	{
		ExecutionContext.Defer().DestroyEntities(EntitiesToDestroy);
	}
}
