// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/MassEngineRenderStateProcessors.h"

#include "AI/MassEngineNavigationProcessors.h"
#include "MassDestroyRenderStateContext.h"
#include "MassEngineTypes.h"
#include "MassExecutionContext.h"
#include "MassISMRenderStateHelper.h"
#include "MassRenderStateHelper.h"
#include "MassSignalSubsystem.h"
#include "MassStaticMeshRenderStateHelper.h"
#include "Mesh/MassEngineMeshFragments.h"
#include "StaticMeshComponentHelper.h"
#include "StaticMeshSceneProxyDesc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEngineRenderStateProcessors)

// ----------------------------------------------------------------------//
// UMassCreateRenderStateProcessor
//----------------------------------------------------------------------//
UMassCreateRenderStateProcessor::UMassCreateRenderStateProcessor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EntityQuery(*this)
{
	bRequiresGameThreadExecution = true; // @todo revisit if the execution is required on game thread
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::CreateRenderState;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::SetupRenderState);
}

void UMassCreateRenderStateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassRenderStateFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddTagRequirement<FMassSceneProxyCreatedTag>(EMassFragmentPresence::None);
}

void UMassCreateRenderStateProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	FRegisterComponentContext BatchingContext(nullptr);

	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&BatchingContext](FMassExecutionContext& Context)
		{
			const TArrayView<FMassRenderStateFragment>& RenderStateFragments = Context.GetMutableFragmentView<FMassRenderStateFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FMassPrimitiveRenderStateHelper& Helper = RenderStateFragments[*EntityIt].GetRenderStateHelper<FMassPrimitiveRenderStateHelper>();
				if (!Helper.IsRenderStateCreated() && Helper.ShouldCreateRenderState())
				{
					if (!BatchingContext.IsInitialized())
					{
						BatchingContext.Initialize(Helper.GetWorld());
					}
					Helper.CreateRenderState(&BatchingContext);
				}
				else
				{
					Context.Defer().AddTag<FMassSceneProxyCreatedTag>(Context.GetEntity(*EntityIt));
				}
			}
		});

	if (BatchingContext.IsInitialized())
	{
		BatchingContext.Process();
#if WITH_EDITOR
		// Make sure any new created hit proxies are taken into account by the ProxyMap
		if (GEditor)
		{
			GEditor->RedrawLevelEditingViewports(/*bInvalidateProxies*/true);
		}
#endif // WITH_EDITOR
	}
}

// ----------------------------------------------------------------------//
// UMassRenderStateDirtyUpdateProcessor
//----------------------------------------------------------------------//
UMassRenderStateDirtyUpdateProcessor::UMassRenderStateDirtyUpdateProcessor(const FObjectInitializer & ObjectInitializer)
	: Super(ObjectInitializer)
{
	bRequiresGameThreadExecution = true; // @todo revisit if the execution is required on game thread
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::CreateRenderState;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::SetupRenderState);
}

void UMassRenderStateDirtyUpdateProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	if (const UWorld* World = GetWorld())
	{
		if (UMassSignalSubsystem* SignalSubsystem = World->GetSubsystem<UMassSignalSubsystem>())
		{
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::TransformChanged);
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::MeshVisualPropertyChanged);
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::MeshChanged);
#if WITH_EDITOR
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::SelectionChanged);
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::LevelEditingStateChanged);
#endif // WITH_EDITOR
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::RenderStateDirty);
		}
	}
}

void UMassRenderStateDirtyUpdateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassRenderStateFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassRenderStateDirtyUpdateProcessor::SignalEntities(
	FMassEntityManager& EntityManager,
	FMassExecutionContext& ExecutionContext,
	FMassSignalNameLookup& EntitySignals)
{
	const UObject* EntityManagerOwner = ExecutionContext.GetEntityManagerChecked().GetOwner();
	check(EntityManagerOwner);

	TArray<FName> SignalsForEntity;
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&EntitySignals, &SignalsForEntity](FMassExecutionContext& Context)
		{
			const TArrayView<FMassRenderStateFragment>& RenderStateFragments = Context.GetMutableFragmentView<FMassRenderStateFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FMassPrimitiveRenderStateHelper& Helper = RenderStateFragments[*EntityIt].GetRenderStateHelper<FMassPrimitiveRenderStateHelper>();
				SignalsForEntity.Reset();
				EntitySignals.GetSignalsForEntity(Context.GetEntity(*EntityIt), SignalsForEntity);

				bool bHandled = false;
				if (SignalsForEntity.Num() == 1)
				{
					if (SignalsForEntity[0] == UE::Mass::Signals::TransformChanged)
					{
						Helper.UpdateTransform();
						bHandled = true;
					}
#if WITH_EDITOR
					else if (SignalsForEntity[0] == UE::Mass::Signals::SelectionChanged)
					{
						Helper.UpdateSelection();
						bHandled = true;
					}
					else if (SignalsForEntity[0] == UE::Mass::Signals::LevelEditingStateChanged)
					{
						Helper.UpdateLevelInstanceEditingState();
						bHandled = true;
					}
#endif // WITH_EDITOR
				}

				if (!bHandled)
				{
					Helper.DestroyRenderState();
					Context.Defer().RemoveTag<FMassSceneProxyCreatedTag>(Context.GetEntity(*EntityIt));
				}
			}
		});
}

// ----------------------------------------------------------------------//
// UMassDestroyRenderStateProcessor
//----------------------------------------------------------------------//
UMassDestroyRenderStateProcessor::UMassDestroyRenderStateProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
	ObservedTypes.Add(FMassRenderStateFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
	bRequiresGameThreadExecution = true; // @todo revisit if the execution is required on game thread
}

void UMassDestroyRenderStateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassRenderStateFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassDestroyRenderStateProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	FMassDestroyRenderStateContext BatchingContext(nullptr);

	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&BatchingContext](FMassExecutionContext& Context)
		{
			const TArrayView<FMassRenderStateFragment>& RenderStateFragments = Context.GetMutableFragmentView<FMassRenderStateFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FMassPrimitiveRenderStateHelper& Helper = RenderStateFragments[*EntityIt].GetRenderStateHelper<FMassPrimitiveRenderStateHelper>();
				if (!BatchingContext.IsInitialized())
				{
					BatchingContext.Initialize(Helper.GetScene());
				}
				Helper.DestroyRenderState(&BatchingContext);
				RenderStateFragments[*EntityIt].DestroyRenderStateHelper();
				Context.Defer().RemoveTag<FMassSceneProxyCreatedTag>(Context.GetEntity(*EntityIt));
			}
		});

	if (BatchingContext.IsInitialized())
	{
		BatchingContext.Process();
	}
}
