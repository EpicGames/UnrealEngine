// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mesh/MassEngineRenderStaticMeshProcessors.h"

#include "AI/MassEngineNavigationProcessors.h"
#include "MassDestroyRenderStateContext.h"
#include "MassEngineTypes.h"
#include "Mass/EntityFragments.h"
#include "MassExecutionContext.h"
#include "MassISMRenderStateHelper.h"
#include "MassRenderStateHelper.h"
#include "MassSignalSubsystem.h"
#include "MassStaticMeshRenderStateHelper.h"
#include "Mesh/MassEngineMeshFragments.h"
#include "StaticMeshComponentHelper.h"
#include "StaticMeshSceneProxyDesc.h"
#include "Mesh/MassEngineMeshUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEngineRenderStaticMeshProcessors)

//----------------------------------------------------------------------//
// UMassBaseStaticMeshSetupRenderStateProcessor
//----------------------------------------------------------------------//
UMassBaseStaticMeshSetupRenderStateProcessor::UMassBaseStaticMeshSetupRenderStateProcessor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EntityQuery(*this)
{
	bRequiresGameThreadExecution = true; // @todo revisit if the execution is required on game thread
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SetupRenderState;
}

void UMassBaseStaticMeshSetupRenderStateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddConstSharedRequirement<FMassStaticMeshFragment>();
	EntityQuery.AddConstSharedRequirement<FMassVisualizationMeshFragment>();
	EntityQuery.AddConstSharedRequirement<FMassOverrideMaterialsFragment>(EMassFragmentPresence::Optional);
#if WITH_EDITOR
	EntityQuery.AddConstSharedRequirement<FMassEditorVisualizationMeshFragment>(EMassFragmentPresence::Optional);
#endif // WITH_EDITOR

	EntityQuery.AddRequirement<FMassRenderPrimitiveFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
	EntityQuery.AddRequirement<FMassRenderStateFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
}

//----------------------------------------------------------------------//
// UMassStaticMeshSetupRenderStateProcessor
//----------------------------------------------------------------------//
void UMassStaticMeshSetupRenderStateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);
	EntityQuery.AddTagRequirement<FMassRenderISMCandidateTag>(EMassFragmentPresence::None);
	EntityQuery.AddRequirement<FMassRenderStaticMeshFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
}

void UMassStaticMeshSetupRenderStateProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&EntityManager](FMassExecutionContext& Context)
		{
			const FMassStaticMeshFragment& StaticMeshFragment = Context.GetConstSharedFragment<FMassStaticMeshFragment>();
			const FMassVisualizationMeshFragment& MeshFragment = Context.GetConstSharedFragment<FMassVisualizationMeshFragment>();
			const FMassOverrideMaterialsFragment* OverrideMaterialsFragment = Context.GetConstSharedFragmentPtr<FMassOverrideMaterialsFragment>();
#if WITH_EDITOR
			const FMassEditorVisualizationMeshFragment* EditorMeshFragment = Context.GetConstSharedFragmentPtr<FMassEditorVisualizationMeshFragment>();
#endif // WITH_EDITOR
			const TConstArrayView<FTransformFragment> TransformFragments = Context.GetFragmentView<FTransformFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FMassRenderPrimitiveFragment RenderPrimitiveFragment;
				RenderPrimitiveFragment.bIsVisible = true;
				RenderPrimitiveFragment.LocalBounds = StaticMeshFragment.Mesh.Get()->GetBounds();
				RenderPrimitiveFragment.WorldBounds = RenderPrimitiveFragment.LocalBounds.TransformBy(TransformFragments[*EntityIt].GetTransform());

				FMassRenderStaticMeshFragment RenderStaticMeshFragment;
				checkf(RenderStaticMeshFragment.StaticMeshSceneProxyDesc, TEXT("Expecting a valid prxy desc here"));

				UE::MassEngine::Mesh::InitializeStaticMeshSceneProxyDescFromFragment(StaticMeshFragment, MeshFragment, TransformFragments[*EntityIt], *RenderStaticMeshFragment.StaticMeshSceneProxyDesc);
#if WITH_EDITOR
				if (EditorMeshFragment)
				{
					UE::MassEngine::Mesh::InitializePrimitiveSceneProxyDescFromEditorFragment(*EditorMeshFragment, *RenderStaticMeshFragment.StaticMeshSceneProxyDesc);
				}
#endif // WITH_EDITOR

				FMassRenderStateFragment RenderStateFragment;
				RenderStateFragment.CreateRenderStateHelper<FMassStaticMeshRenderStateHelper>(Context.GetEntity(*EntityIt), &EntityManager, RenderPrimitiveFragment, OverrideMaterialsFragment, RenderStaticMeshFragment);

				// Add new fragments
				Context.Defer().PushCommand<FMassCommandAddFragmentInstances>(Context.GetEntity(*EntityIt), MoveTemp(RenderPrimitiveFragment), MoveTemp(RenderStaticMeshFragment), MoveTemp(RenderStateFragment));
			}
		});
}

//----------------------------------------------------------------------//
// UMassStaticMeshUpdateRenderStateProcessor
//----------------------------------------------------------------------//
UMassStaticMeshUpdateRenderStateProcessor::UMassStaticMeshUpdateRenderStateProcessor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bRequiresGameThreadExecution = true; // @todo revisit if the execution is required on game thread
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SetupRenderState;
}

void UMassStaticMeshUpdateRenderStateProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	if (const UWorld* World = GetWorld())
	{
		if (UMassSignalSubsystem* SignalSubsystem = World->GetSubsystem<UMassSignalSubsystem>())
		{
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::TransformChanged);
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::MeshVisualPropertyChanged);
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::MeshChanged);
#if WITH_EDITOR
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::SelectionChanged);
			SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::LevelEditingStateChanged);
#endif // WITH_EDITOR
		}
	}
}

void UMassStaticMeshUpdateRenderStateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddConstSharedRequirement<FMassStaticMeshFragment>();
	EntityQuery.AddConstSharedRequirement<FMassVisualizationMeshFragment>();
#if WITH_EDITOR
	EntityQuery.AddConstSharedRequirement<FMassEditorVisualizationMeshFragment>(EMassFragmentPresence::Optional);
#endif // WITH_EDITOR
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);

	EntityQuery.AddRequirement<FMassRenderPrimitiveFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassRenderStaticMeshFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassStaticMeshUpdateRenderStateProcessor::SignalEntities(
	FMassEntityManager& EntityManager,
	FMassExecutionContext& ExecutionContext,
	FMassSignalNameLookup& EntitySignals)
{
	const UObject* EntityManagerOwner = ExecutionContext.GetEntityManagerChecked().GetOwner();
	check(EntityManagerOwner);

	TArray<FName> SignalsForEntity;
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&EntitySignals, &SignalsForEntity](FMassExecutionContext& Context)
		{
			const FMassStaticMeshFragment& StaticMeshFragment = Context.GetConstSharedFragment<FMassStaticMeshFragment>();
			const FMassVisualizationMeshFragment& MeshFragment = Context.GetConstSharedFragment<FMassVisualizationMeshFragment>();
#if WITH_EDITOR
			const FMassEditorVisualizationMeshFragment* EditorMeshFragment = Context.GetConstSharedFragmentPtr<FMassEditorVisualizationMeshFragment>();
#endif // WITH_EDITOR
			const TConstArrayView<FTransformFragment> TransformFragments = Context.GetFragmentView<FTransformFragment>();

			const TArrayView<FMassRenderPrimitiveFragment>& RenderPrimitiveFragments = Context.GetMutableFragmentView<FMassRenderPrimitiveFragment>();
			const TArrayView<FMassRenderStaticMeshFragment>& RenderStaticMeshFragments = Context.GetMutableFragmentView<FMassRenderStaticMeshFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				SignalsForEntity.Reset();
				EntitySignals.GetSignalsForEntity(Context.GetEntity(*EntityIt), SignalsForEntity);

				const bool bTransformChanged = SignalsForEntity.Contains(UE::Mass::Signals::TransformChanged);
				const bool bVisualPropertyChanged = SignalsForEntity.Contains(UE::Mass::Signals::MeshVisualPropertyChanged);
				const bool bMeshChanged = SignalsForEntity.Contains(UE::Mass::Signals::MeshChanged);
#if WITH_EDITOR
				const bool bSelectionChanged = SignalsForEntity.Contains(UE::Mass::Signals::SelectionChanged);
				const bool bLevelEditingStateChanged = SignalsForEntity.Contains(UE::Mass::Signals::LevelEditingStateChanged);
#endif // WITH_EDITOR

				if (bTransformChanged)
				{
					RenderPrimitiveFragments[*EntityIt].bIsVisible = true;
					RenderPrimitiveFragments[*EntityIt].LocalBounds = StaticMeshFragment.Mesh.Get()->GetBounds();
					RenderPrimitiveFragments[*EntityIt].WorldBounds = RenderPrimitiveFragments[*EntityIt].LocalBounds.TransformBy(TransformFragments[*EntityIt].GetTransform());
				}

				checkf(RenderStaticMeshFragments[*EntityIt].StaticMeshSceneProxyDesc, TEXT("Expecting a valid prxy desc here"));
				if (bTransformChanged || bMeshChanged || bVisualPropertyChanged)
				{
					UE::MassEngine::Mesh::InitializeStaticMeshSceneProxyDescFromFragment(StaticMeshFragment, MeshFragment, TransformFragments[*EntityIt], *RenderStaticMeshFragments[*EntityIt].StaticMeshSceneProxyDesc);
				}
#if WITH_EDITOR
				if (EditorMeshFragment && (bVisualPropertyChanged || bSelectionChanged || bLevelEditingStateChanged) )
				{
					UE::MassEngine::Mesh::InitializePrimitiveSceneProxyDescFromEditorFragment(*EditorMeshFragment, *RenderStaticMeshFragments[*EntityIt].StaticMeshSceneProxyDesc);
				}
#endif // WITH_EDITOR
			}

		});
}

//----------------------------------------------------------------------//
// UMassBaseStaticMeshDestroyRenderStateProcessor
//----------------------------------------------------------------------//
UMassBaseStaticMeshDestroyRenderStateProcessor::UMassBaseStaticMeshDestroyRenderStateProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
	ObservedTypes.Add(FTransformFragment::StaticStruct());
	ObservedTypes.Add(FMassStaticMeshFragment::StaticStruct());
	ObservedTypes.Add(FMassVisualizationMeshFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
}

void UMassBaseStaticMeshDestroyRenderStateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	// @Todo, this would prevent the observer to be called more than once when the fragments are removed in multiple calls, 
	// but we need to sort out the Observer manager LockCount batcher before. Otherwise we get called after the fragments are 
	// removed and the query will never run.
	//EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	//EntityQuery.AddConstSharedRequirement<FMassStaticMeshFragment>(EMassFragmentPresence::All);
	//EntityQuery.AddConstSharedRequirement<FMassVisualizationMeshFragment>(EMassFragmentPresence::All);
}

//----------------------------------------------------------------------//
// UMassStaticMeshDestroyRenderStateProcessor
//----------------------------------------------------------------------//
void UMassStaticMeshDestroyRenderStateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);

	EntityQuery.AddRequirement<FMassRenderStateFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRenderPrimitiveFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRenderStaticMeshFragment>(EMassFragmentAccess::ReadOnly);
}

void UMassStaticMeshDestroyRenderStateProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[](FMassExecutionContext& Context)
		{
			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				Context.Defer().PushCommand<FMassCommandRemoveFragments<FMassRenderStateFragment, FMassRenderPrimitiveFragment, FMassRenderStaticMeshFragment>>(Context.GetEntity(*EntityIt));
			}
		});
}
