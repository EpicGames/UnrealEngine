// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassISMRenderStateHelper.h"

#include "InstanceData/InstanceDataHelpers.h"
#include "InstancedStaticMeshComponentHelper.h"
#include "MassEntityManager.h"
#include "Mesh/MassEngineMeshFragments.h"
#include "NaniteSceneProxy.h"
#include "SceneInterface.h"

FMassISMRenderStateHelper::FMassISMRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager, const FMassRenderPrimitiveFragment& RenderPrimitiveFragment, const FMassOverrideMaterialsFragment* OverrideMaterialsFragment, const FMassRenderISMFragment& RenderStaticMeshFragment)
	: Super(InEntityHandle, InEntityManager, RenderPrimitiveFragment, OverrideMaterialsFragment)
{
}

void FMassISMRenderStateHelper::ResetSceneProxyDescUnsupportedProperties()
{
	Super::ResetSceneProxyDescUnsupportedProperties();

	FMassRenderISMFragment& RenderISMFragment = GetMutableRenderISMFragment();

	checkf(RenderISMFragment.InstancedStaticMeshSceneProxyDesc, TEXT("Expecting a valid scene proxy desc"));
	RenderISMFragment.InstancedStaticMeshSceneProxyDesc->InstanceDataSceneProxy = nullptr;
#if WITH_EDITOR
	RenderISMFragment.InstancedStaticMeshSceneProxyDesc->bHasSelectedInstances = false;
#endif// WITH_EDITOR 
}

TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe>& FMassISMRenderStateHelper::BuildInstanceData()
{
	FInstanceSceneDataBuffers InstanceSceneDataBuffers{};
	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	auto View = InstanceSceneDataBuffers.BeginWriteAccess(AccessTag);
	
	// PrimitiveLocalToWorld
	InstanceSceneDataBuffers.SetPrimitiveLocalToWorld(GetRenderMatrix(), AccessTag);
	
	// InstanceLocalBounds
	const FPrimitiveMaterialPropertyDescriptor PrimitiveMaterialDesc = GetUsedMaterialPropertyDesc(GetScene()->GetShaderPlatform());
	const float LocalAbsMaxDisplacement = FMath::Max(-PrimitiveMaterialDesc.MinMaxMaterialDisplacement.X, PrimitiveMaterialDesc.MinMaxMaterialDisplacement.Y) + PrimitiveMaterialDesc.MaxWorldPositionOffsetDisplacement;
	const FVector3f PadExtent = FISMCInstanceDataSceneProxy::GetLocalBoundsPadExtent(View.PrimitiveToRelativeWorld, LocalAbsMaxDisplacement);
	FRenderBounds InstanceLocalBounds = GetStaticMesh()->GetBounds();
	InstanceLocalBounds.Min -= PadExtent;
	InstanceLocalBounds.Max += PadExtent;
	check(!View.Flags.bHasPerInstanceLocalBounds);
	View.InstanceLocalBounds.Add(InstanceLocalBounds);

	const FMassRenderISMFragment& RenderISMFragment = GetRenderISMFragment();

	// LocalToPrimitiveRelativeWorld
	View.InstanceToPrimitiveRelative.Reserve(RenderISMFragment.PerInstanceSMData.Num());
	for (const auto& SM : RenderISMFragment.PerInstanceSMData)
	{
		FRenderTransform InstanceToPrimitive = SM.Transform;
		FRenderTransform LocalToPrimitiveRelativeWorld = InstanceToPrimitive * View.PrimitiveToRelativeWorld;
		LocalToPrimitiveRelativeWorld.Orthogonalize();
		View.InstanceToPrimitiveRelative.Add(LocalToPrimitiveRelativeWorld);
	}

	// InstanceCustomData
	View.InstanceCustomData = RenderISMFragment.PerInstanceSMCustomData;
	View.NumCustomDataFloats = RenderISMFragment.PerInstanceSMCustomData.Num() && RenderISMFragment.PerInstanceSMData.Num() ? RenderISMFragment.PerInstanceSMCustomData.Num() / RenderISMFragment.PerInstanceSMData.Num() : 0;
	View.Flags.bHasPerInstanceCustomData = PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceCustomData && View.NumCustomDataFloats != 0;
	if (!View.Flags.bHasPerInstanceCustomData)
	{
		View.NumCustomDataFloats = 0;
		View.InstanceCustomData.Reset();
	}

	// InstanceRandomIDs
	View.Flags.bHasPerInstanceRandom = PrimitiveMaterialDesc.bAnyMaterialHasPerInstanceRandom && (RenderISMFragment.PerInstanceSMData.Num() > 0);
	if (View.Flags.bHasPerInstanceRandom)
	{
		InstanceRandomIDs.SetNumUninitialized(RenderISMFragment.PerInstanceSMData.Num());
		FInstanceDataHelpers::GenerateInstanceRandomIDs(RenderISMFragment.InstancingRandomSeed, RenderISMFragment.AdditionalRandomSeeds, InstanceRandomIDs);
	}
	View.InstanceRandomIDs = InstanceRandomIDs;

#if WITH_EDITOR
	// Hit proxies
	if (RenderISMFragment.PerInstanceHitProxy.Num())
	{
		const FMassEditorVisualizationMeshFragment* EditorMeshFragment = GetEditorMeshFragment();
		const bool bSelected = EditorMeshFragment ? EditorMeshFragment->bShouldRenderSelected : false;

		View.Flags.bHasPerInstanceEditorData = true;
		View.InstanceEditorData.Reserve(RenderISMFragment.PerInstanceHitProxy.Num());
		for (HHitProxy* HitProxy : RenderISMFragment.PerInstanceHitProxy)
		{
			View.InstanceEditorData.Add(FInstanceEditorData::Pack(HitProxy->Id.GetColor(), bSelected));
		}
	}
#endif // WITH_EDITOR
	InstanceSceneDataBuffers.EndWriteAccess(AccessTag);
	InstanceSceneDataBuffers.ValidateData();

	DataProxy = MakeShared<FInstanceDataSceneProxy, ESPMode::ThreadSafe>(MoveTemp(InstanceSceneDataBuffers));
	return DataProxy;
}

FPrimitiveSceneProxy* FMassISMRenderStateHelper::CreateStaticMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	FMassRenderISMFragment& RenderISMFragment = GetMutableRenderISMFragment();

	check(GetWorld());
	check(RenderISMFragment.InstancedStaticMeshSceneProxyDesc);
	check(RenderISMFragment.InstancedStaticMeshSceneProxyDesc->Scene);
	check(RenderISMFragment.PerInstanceSMData.Num() > 0);

	RenderISMFragment.InstancedStaticMeshSceneProxyDesc->InstanceDataSceneProxy = BuildInstanceData();
	if (bCreateNanite)
	{
		PrimitiveSceneData.SceneProxy = ::new Nanite::FSceneProxy(NaniteMaterials, *RenderISMFragment.InstancedStaticMeshSceneProxyDesc);
	}
	else
	{
		PrimitiveSceneData.SceneProxy = ::new FInstancedStaticMeshSceneProxy(*RenderISMFragment.InstancedStaticMeshSceneProxyDesc, RenderISMFragment.InstancedStaticMeshSceneProxyDesc->FeatureLevel);
	}
	return PrimitiveSceneData.SceneProxy;
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void FMassISMRenderStateHelper::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	// @TODO Implement this
	//FInstancedStaticMeshComponentHelper::CollectPSOPrecacheData(*this, BasePrecachePSOParams, OutParams);
}
#endif


const FMassRenderISMFragment& FMassISMRenderStateHelper::GetRenderISMFragment() const
{
	return GetEntityManager().GetFragmentDataChecked<FMassRenderISMFragment>(EntityHandle);
}

FMassRenderISMFragment& FMassISMRenderStateHelper::GetMutableRenderISMFragment()
{
	return GetEntityManager().GetFragmentDataChecked<FMassRenderISMFragment>(EntityHandle);
}

void FMassISMRenderStateHelper::InitializeSceneProxyDescDynamicProperties()
{
	FMassBaseStaticMeshRenderStateHelper::InitializeSceneProxyDescDynamicProperties();

#if WITH_EDITOR
	FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();

	FMassRenderISMFragment& RenderISMFragment = GetMutableRenderISMFragment();
	SceneProxyDesc.bHasPerInstanceHitProxies = RenderISMFragment.PerInstanceHitProxy.Num() > 0;
#endif // WITH_EDITOR
}

FPrimitiveSceneProxyDesc& FMassISMRenderStateHelper::GetSceneProxyDesc()
{
	return GetStaticMeshSceneProxyDesc();
}

const FPrimitiveSceneProxyDesc& FMassISMRenderStateHelper::GetSceneProxyDesc() const
{
	return GetStaticMeshSceneProxyDesc();
}

FStaticMeshSceneProxyDesc& FMassISMRenderStateHelper::GetStaticMeshSceneProxyDesc()
{
	FMassRenderISMFragment& RenderISMFragment = GetMutableRenderISMFragment();
	checkf(RenderISMFragment.InstancedStaticMeshSceneProxyDesc, TEXT("Expecting a valid scene proxy desc"));
	return *RenderISMFragment.InstancedStaticMeshSceneProxyDesc;
}

const FStaticMeshSceneProxyDesc& FMassISMRenderStateHelper::GetStaticMeshSceneProxyDesc() const
{
	const FMassRenderISMFragment& RenderISMFragment = GetRenderISMFragment();
	checkf(RenderISMFragment.InstancedStaticMeshSceneProxyDesc, TEXT("Expecting a valid scene proxy desc"));
	return *RenderISMFragment.InstancedStaticMeshSceneProxyDesc;
}
