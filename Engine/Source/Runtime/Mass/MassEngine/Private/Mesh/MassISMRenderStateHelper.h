// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStaticMeshRenderStateHelper.h"
#include "InstancedStaticMeshSceneProxyDesc.h"

struct FMassRenderISMFragment;

/**
 * Helper to handle all the communication to the renderer from Mass for an instanced static mesh
 */
struct FMassISMRenderStateHelper : public FMassBaseStaticMeshRenderStateHelper
{
public:
	using Super = FMassBaseStaticMeshRenderStateHelper ;

	FMassISMRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager, const FMassRenderPrimitiveFragment& RenderPrimitiveFragment, const FMassOverrideMaterialsFragment* OverrideMaterialsFragment, const FMassRenderISMFragment& RenderStaticMeshFragment);
	virtual ~FMassISMRenderStateHelper() override = default;

protected:
	//~ Begin FMassPrimitiveRenderStateHelper interface
	virtual FPrimitiveSceneProxyDesc& GetSceneProxyDesc() override;
	virtual const FPrimitiveSceneProxyDesc& GetSceneProxyDesc() const override;
	virtual void ResetSceneProxyDescUnsupportedProperties() override;
	//~ End FMassPrimitiveRenderStateHelper interface

	//~ Begin FMassBaseStaticMeshRenderStateHelper interface
	virtual FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() override;
	virtual const FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() const override;
	virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite) override;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
#endif
	//~ End FMassBaseStaticMeshRenderStateHelper interface

	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe>& BuildInstanceData();

	const FMassRenderISMFragment& GetRenderISMFragment() const;
	FMassRenderISMFragment& GetMutableRenderISMFragment();

protected:

	virtual void InitializeSceneProxyDescDynamicProperties() override;

	TSharedPtr<FInstanceDataSceneProxy, ESPMode::ThreadSafe> DataProxy{};
	TArray<float> InstanceRandomIDs;

	friend class FInstancedStaticMeshComponentHelper;
};
