// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassMeshRenderStateHelper.h"

#include "MassEntityManager.h"
#include "MeshComponentHelper.h"
#include "Mesh/MassEngineMeshFragments.h"

FMassMeshRenderStateHelper::FMassMeshRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager, const FMassRenderPrimitiveFragment& RenderPrimitiveFragment, const FMassOverrideMaterialsFragment* OverrideMaterialsFragment)
	: Super(InEntityHandle, InEntityManager, RenderPrimitiveFragment)
{
}	

void FMassMeshRenderStateHelper::GetMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const
{
	FMeshComponentHelper::GetMaterialSlotsOverlayMaterial(*this, OutMaterialSlotOverlayMaterials);
}

FMaterialRelevance FMassMeshRenderStateHelper::GetMaterialRelevance(EShaderPlatform InShaderPlatform) const
{
	return FMeshComponentHelper::GetMaterialRelevance(*this, InShaderPlatform);
}

const FMassOverrideMaterialsFragment* FMassMeshRenderStateHelper::GetOverrideMaterialsFragment() const
{
	return GetEntityManager().GetConstSharedFragmentDataPtr<FMassOverrideMaterialsFragment>(EntityHandle);
}

const FMassVisualizationMeshFragment& FMassMeshRenderStateHelper::GetMeshFragment() const
{
	return GetEntityManager().GetConstSharedFragmentDataChecked<FMassVisualizationMeshFragment>(EntityHandle);
}

#if WITH_EDITOR
const FMassEditorVisualizationMeshFragment* FMassMeshRenderStateHelper::GetEditorMeshFragment() const
{
	return GetEntityManager().GetConstSharedFragmentDataPtr<FMassEditorVisualizationMeshFragment>(EntityHandle);
}
#endif // WITH_EDITOR

TConstArrayView<TObjectPtr<UMaterialInterface>> FMassMeshRenderStateHelper::GetOverrideMaterials() const
{
	if (const FMassOverrideMaterialsFragment* OverrideMaterialFragment = GetOverrideMaterialsFragment())
	{
		return OverrideMaterialFragment->OverrideMaterials;
	}
	return TConstArrayView<TObjectPtr<UMaterialInterface>>();
}
