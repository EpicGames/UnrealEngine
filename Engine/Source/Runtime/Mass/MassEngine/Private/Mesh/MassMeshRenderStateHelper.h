// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassPrimitiveRenderStateHelper.h"

struct FMassEditorVisualizationMeshFragment;
struct FMassVisualizationMeshFragment;
class UMaterialInterface;
struct FMaterialRelevance;
struct FMassOverrideMaterialsFragment;

/**
 * Helper to handle all the communication to the renderer from Mass for all type of meshes 
 */
struct FMassMeshRenderStateHelper : public FMassPrimitiveRenderStateHelper
{
public:
	using Super = FMassPrimitiveRenderStateHelper;

	FMassMeshRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager, const FMassRenderPrimitiveFragment& RenderPrimitiveFragment, const FMassOverrideMaterialsFragment* OverrideMaterialsFragment);
	virtual ~FMassMeshRenderStateHelper() override = default;

	FMaterialRelevance GetMaterialRelevance(EShaderPlatform InShaderPlatform) const;

	void GetMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const;

	//Get the asset material slot overlay material
	virtual void GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& AssetMaterialSlotOverlayMaterials) const = 0;

	virtual const TArray<TObjectPtr<UMaterialInterface>>& GetComponentMaterialSlotsOverlayMaterial() const = 0;
	virtual UMaterialInterface* GetOverlayMaterial() const = 0;

	const FMassOverrideMaterialsFragment* GetOverrideMaterialsFragment() const;
	const FMassVisualizationMeshFragment& GetMeshFragment() const;
#if WITH_EDITOR
	const FMassEditorVisualizationMeshFragment* GetEditorMeshFragment() const;
#endif // WITH_EDITOR

	TConstArrayView<TObjectPtr<UMaterialInterface>> GetOverrideMaterials() const;
};