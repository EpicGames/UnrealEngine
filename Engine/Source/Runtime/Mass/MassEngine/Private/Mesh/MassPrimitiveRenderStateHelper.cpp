// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassPrimitiveRenderStateHelper.h"

#include "MassDestroyRenderStateContext.h"
#include "Mass/EntityFragments.h"
#include "MassEntityManager.h"
#include "Mesh/MassEngineMeshFragments.h"
#include "PrimitiveComponentHelper.h"
#include "PrimitiveSceneDesc.h"
#include "PrimitiveSceneProxyDesc.h"
#include "SceneInterface.h"
#include "UnrealEngine.h"
#include "Elements/Framework/TypedElementHandle.h"


#if WITH_EDITOR
#include "ObjectCacheEventSink.h"
#endif //WITH_EDITOR

//----------------------------------------------------------------------//
// FMassPrimitiveRenderStateHelper
//----------------------------------------------------------------------//
FMassPrimitiveRenderStateHelper::FMassPrimitiveRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager, const FMassRenderPrimitiveFragment& RenderPrimitiveFragment)
	: Super(InEntityHandle, InEntityManager)
{
	// Copying data from fragment as scene proxy decs take a view on this array
	CustomPrimitiveData = RenderPrimitiveFragment.CustomPrimitiveData;
}

const FMassRenderPrimitiveFragment& FMassPrimitiveRenderStateHelper::GetRenderPrimitiveFragment() const
{
	return GetEntityManager().GetFragmentDataChecked<FMassRenderPrimitiveFragment>(EntityHandle);
}

FMassRenderPrimitiveFragment& FMassPrimitiveRenderStateHelper::GetMutableRenderPrimitiveFragment()
{
	return GetEntityManager().GetFragmentDataChecked<FMassRenderPrimitiveFragment>(EntityHandle);
}

void FMassPrimitiveRenderStateHelper::ResetSceneProxyDescUnsupportedProperties()
{
	// Unsupported properties
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	//SceneProxyDesc.bLevelInstanceEditingState = false; //@Todo FastGeo will need to turn this off outside of this method
	//SceneProxyDesc.bSelectable = false; //@Todo FastGeo will need to turn this off outside of this method
	SceneProxyDesc.bUseEditorCompositing = false;
	SceneProxyDesc.bIsBeingMovedByEditor = false;
	//SceneProxyDesc.bSelected = false; //@Todo FastGeo will need to turn this off outside of this method
	//SceneProxyDesc.bIndividuallySelected = false; //@Todo FastGeo will need to turn this off outside of this method
	//SceneProxyDesc.bShouldRenderSelected = false; //@Todo FastGeo will need to turn this off outside of this method
	SceneProxyDesc.bWantsEditorEffects = false;
	SceneProxyDesc.bIsHiddenEd = false;
	SceneProxyDesc.bIsOwnerEditorOnly = false;
#if WITH_EDITOR
	SceneProxyDesc.bIsOwnedByFoliage = false;
#endif // WITH_EDITOR
	SceneProxyDesc.HiddenEditorViews = 0;
#if WITH_EDITOR
	SceneProxyDesc.OverlayColor = FColor(EForceInit::ForceInitToZero);
#endif // WITH_EDITOR
	SceneProxyDesc.Component = nullptr;

	// Properties that will be initialized by InitializeSceneProxyDescDynamicProperties
	SceneProxyDesc.ComponentId = FPrimitiveComponentId();
	SceneProxyDesc.StatId = TStatId();
	SceneProxyDesc.Owner = nullptr;
	SceneProxyDesc.World = nullptr;
	SceneProxyDesc.CustomPrimitiveData = nullptr;
	SceneProxyDesc.Scene = nullptr;
	SceneProxyDesc.PrimitiveComponentInterface = nullptr;
	SceneProxyDesc.FeatureLevel = ERHIFeatureLevel::Num;
	SceneProxyDesc.RuntimeVirtualTextures = TArrayView<URuntimeVirtualTexture*>();
	SceneProxyDesc.bIsVisible = false;
	SceneProxyDesc.bUsePSOPrecacheFallbackMaterial = false;
#if MESH_DRAW_COMMAND_STATS
	SceneProxyDesc.MeshDrawCommandStatsCategory = NAME_None;
#endif // MESH_DRAW_COMMAND_STATS
}

bool FMassPrimitiveRenderStateHelper::IsDrawnInGame() const
{
	// Drawn in game must consider both the component bIsVisible flag AND the bIsHidden flag (which actually originates from the actor bHiddenInGame property)
	// This logic mimics what is done to initialize FPrimitiveSceneProxy::DrawInGame
	return GetSceneProxyDesc().bIsVisible && !GetSceneProxyDesc().bIsHidden;
}

EComponentMobility::Type FMassPrimitiveRenderStateHelper::GetMobility() const
{
	return GetSceneProxyDesc().Mobility;
}

void FMassPrimitiveRenderStateHelper::UpdateVisibility()
{
	const FMassRenderPrimitiveFragment& RenderPrimitiveFragment = GetRenderPrimitiveFragment();

	// Update SceneProxyDesc.bIsVisible as it's dependant on component and component cluster visibility
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	SceneProxyDesc.bIsVisible = RenderPrimitiveFragment.bIsVisible;
}

void FMassPrimitiveRenderStateHelper::UpdateTransform()
{
	FMassRenderPrimitiveFragment& RenderPrimitiveFragment = GetMutableRenderPrimitiveFragment();

	FSceneInterface* Scene = GetScene();
	checkf(Scene, TEXT("Expecting a scene if this method is called"));

	SceneDesc.RenderMatrix = GetRenderMatrix();
	SceneDesc.AttachmentRootPosition = GetTransform().GetTranslation();
	SceneDesc.LocalBounds = RenderPrimitiveFragment.LocalBounds;
	SceneDesc.Bounds = GetBounds();

	Scene->UpdatePrimitiveTransform(&SceneDesc);
}

#if WITH_EDITOR
void FMassPrimitiveRenderStateHelper::UpdateSelection()
{
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	if (PrimitiveSceneData.SceneProxy)
	{
		PrimitiveSceneData.SceneProxy->SetSelection_GameThread(SceneProxyDesc.ShouldRenderSelected());
	}
}

void FMassPrimitiveRenderStateHelper::UpdateLevelInstanceEditingState()
{
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	if (PrimitiveSceneData.SceneProxy)
	{
		PrimitiveSceneData.SceneProxy->SetLevelInstanceEditingState_GameThread(SceneProxyDesc.GetLevelInstanceEditingState());
	}
}
#endif// WITH_EDITOR

void FMassPrimitiveRenderStateHelper::InitializeSceneProxyDescDynamicProperties()
{
	check(GetWorld());
	check(GetScene());

	ResetSceneProxyDescUnsupportedProperties();

	// Initialize non-serialized properties
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	SceneProxyDesc.ComponentId = GetPrimitiveSceneId();
	UObject const* AdditionalStatObjectPtr = AdditionalStatObject();
	SceneProxyDesc.StatId = AdditionalStatObjectPtr ? AdditionalStatObjectPtr->GetStatID(true) : TStatId();
	SceneProxyDesc.Owner = GetOwner();
#if !WITH_STATE_STREAM
	SceneProxyDesc.World = GetWorld();
#else
	SceneProxyDesc.World = nullptr;
#endif // !WITH_STATE_STREAM
	SceneProxyDesc.CustomPrimitiveData = &CustomPrimitiveData;
	SceneProxyDesc.Scene = GetScene();
	SceneProxyDesc.PrimitiveComponentInterface = this;
	SceneProxyDesc.FeatureLevel = SceneProxyDesc.Scene->GetFeatureLevel();
	TConstArrayView<URuntimeVirtualTexture*> RuntimeVirtualTextures = GetRuntimeVirtualTextures();
	SceneProxyDesc.RuntimeVirtualTextures = (TArrayView<URuntimeVirtualTexture*>&)RuntimeVirtualTextures; // ugly cast, but just making it work for now. It would mean to change the SceneProxyDesc to fix this.
	SceneProxyDesc.bUsePSOPrecacheFallbackMaterial = UsePSOPrecacheFallbackMaterial();
#if MESH_DRAW_COMMAND_STATS
	static FName NAME_FastGeoPrimitiveComponent(TEXT("FastGeoPrimitiveComponent"));
	SceneProxyDesc.MeshDrawCommandStatsCategory = NAME_FastGeoPrimitiveComponent;
#endif // MESH_DRAW_COMMAND_STATS
	//check(SceneProxyDesc.bIsInstancedStaticMesh == (this->IsA<FFastGeoInstancedStaticMeshComponent>() || this->IsA<FFastGeoProceduralISMComponent>()));
	UpdateVisibility();
}

FPrimitiveSceneDesc FMassPrimitiveRenderStateHelper::BuildSceneDesc()
{
	FMassRenderPrimitiveFragment& RenderPrimitiveFragment = GetMutableRenderPrimitiveFragment();

	check(GetSceneProxy());

	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	SceneDesc.SceneProxy = GetSceneProxy();
	SceneDesc.ProxyDesc = &SceneProxyDesc;
	SceneDesc.PrimitiveSceneData = &PrimitiveSceneData;
#if WITH_EDITOR
	SceneDesc.PrimitiveComponentInterface = this;
#endif
	SceneDesc.RenderMatrix = GetRenderMatrix();
	SceneDesc.AttachmentRootPosition = GetTransform().GetTranslation();
	SceneDesc.LocalBounds = RenderPrimitiveFragment.LocalBounds;
	SceneDesc.Bounds = GetBounds();
	SceneDesc.Mobility = SceneProxyDesc.Mobility;

	return SceneDesc;
}

void FMassPrimitiveRenderStateHelper::CreateRenderState(FRegisterComponentContext* Context)
{
	//FWriteScopeLock WriteLock(*Lock.Get());
	check(ProxyState == EProxyCreationState::None);
	ProxyState = EProxyCreationState::Creating;
	bRenderStateDirty = false;

#if WITH_EDITOR
	NotifyRenderStateChanged();
#endif
	
	FSceneInterface* Scene = GetScene();
	check(Scene);

	ESceneProxyCreationError Error = ESceneProxyCreationError::None;
	if (FPrimitiveSceneProxy* SceneProxy = CreateSceneProxy(&Error))
	{
		//SceneProxy->SetPrimitiveColor(GetDebugColor());
		check(GetSceneProxy());
		FPrimitiveSceneDesc Desc = BuildSceneDesc();
		Scene->AddPrimitive(&Desc);

		ProxyState = EProxyCreationState::Created;
	}
	else if (Error == ESceneProxyCreationError::WaitingPSOs)
	{
		ProxyState = EProxyCreationState::Delayed;
	}
	else
	{
		ProxyState = EProxyCreationState::None;
	}
}

FString FMassPrimitiveRenderStateHelper::GetName() const
{
	return GetNameSafe(GetEntityManager().GetOwner());
}

FString FMassPrimitiveRenderStateHelper::GetFullName() const
{
	return GetFullNameSafe(GetEntityManager().GetOwner());
}

void FMassPrimitiveRenderStateHelper::DestroyRenderState(FMassDestroyRenderStateContext* Context)
{
	//FWriteScopeLock WriteLock(*Lock.Get());
	if (GetSceneProxy())
	{
		FSceneInterface* Scene = GetScene();
		check(Scene);

		check(ProxyState == EProxyCreationState::Created);

		FMassDestroyRenderStateContext::DestroyProxy(Context, GetSceneProxy(), Scene);

		PrimitiveSceneData.SceneProxy = nullptr;

		ProxyState = EProxyCreationState::None;

		bRenderStateDirty = false;

#if WITH_EDITOR
		NotifyRenderStateChanged();
#endif
	}
	else
	{
		check(ProxyState == EProxyCreationState::None || ProxyState == EProxyCreationState::Delayed);
		ProxyState = EProxyCreationState::None;
	}
}

#if WITH_EDITOR
void FMassPrimitiveRenderStateHelper::NotifyRenderStateChanged()
{
	FObjectCacheEventSink::NotifyRenderStateChanged_Concurrent(this);
}
#endif

bool FMassPrimitiveRenderStateHelper::IsRenderStateDirty() const
{
	return Super::IsRenderStateDirty();
}

bool FMassPrimitiveRenderStateHelper::IsRenderStateCreated() const
{
	return Super::IsRenderStateCreated();
}

bool FMassPrimitiveRenderStateHelper::ShouldCreateRenderState() const
{
	const FMassRenderPrimitiveFragment& RenderPrimitiveFragment = GetRenderPrimitiveFragment();

	if (!FApp::CanEverRender())
	{
		return false;
	}

	// If the detail mode setting allows it, add it to the scene.
	const bool bDetailModeAllowsRendering = RenderPrimitiveFragment.DetailMode <= GetCachedScalabilityCVars().DetailMode;
	if (!bDetailModeAllowsRendering)
	{
		return false;
	}

	const FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	const bool bShouldCreateRenderState = (RenderPrimitiveFragment.bIsVisible && !SceneProxyDesc.bIsHidden) || SceneProxyDesc.bCastHiddenShadow || SceneProxyDesc.bAffectIndirectLightingWhileHidden || SceneProxyDesc.bRayTracingFarField;
	return bShouldCreateRenderState;
}

bool FMassPrimitiveRenderStateHelper::IsRegistered() const
{
	// If the Mass entity is created it is because the element is registered in the world/scene
	return true;
}

bool FMassPrimitiveRenderStateHelper::IsUnreachable() const
{
	const UObject* OwnerPtr = GetEntityManager().GetOwner();
	return !OwnerPtr || OwnerPtr->IsUnreachable();
}

bool FMassPrimitiveRenderStateHelper::IsStaticMobility() const
{
	return true;
}

bool FMassPrimitiveRenderStateHelper::IsMipStreamingForced() const
{
	return false;
}

UWorld* FMassPrimitiveRenderStateHelper::GetWorld() const
{
	return Super::GetWorld();
}

FSceneInterface* FMassPrimitiveRenderStateHelper::GetScene() const
{
	return Super::GetScene();
}

TConstArrayView<URuntimeVirtualTexture*> FMassPrimitiveRenderStateHelper::GetRuntimeVirtualTextures() const 
{
	const FMassRenderPrimitiveFragment& RenderPrimitiveFragment = GetRenderPrimitiveFragment();
	return RenderPrimitiveFragment.RuntimeVirtualTextures;
}

bool FMassPrimitiveRenderStateHelper::IsFirstPersonRelevant() const
{
	return GetSceneProxyDesc().IsFirstPersonRelevant();
}

FPrimitiveComponentId FMassPrimitiveRenderStateHelper::GetPrimitiveSceneId() const 
{ 
	return PrimitiveSceneData.PrimitiveSceneId;
}

FPrimitiveSceneProxy* FMassPrimitiveRenderStateHelper::GetSceneProxy() const
{
	return PrimitiveSceneData.SceneProxy;
}

void FMassPrimitiveRenderStateHelper::MarkRenderStateDirty()
{
	Super::MarkRenderStateDirty();
}

void FMassPrimitiveRenderStateHelper::DestroyRenderState()
{
	DestroyRenderState(nullptr);
}

FTransform FMassPrimitiveRenderStateHelper::GetTransform() const
{
	const FTransformFragment& TransformFragment = GetEntityManager().GetFragmentDataChecked<FTransformFragment>(EntityHandle);
	return TransformFragment.GetTransform();
}

const FBoxSphereBounds& FMassPrimitiveRenderStateHelper::GetBounds() const
{
	const FMassRenderPrimitiveFragment& RenderPrimitiveFragment = GetRenderPrimitiveFragment();
	return RenderPrimitiveFragment.WorldBounds;
}

void FMassPrimitiveRenderStateHelper::GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const
{
}

UObject* FMassPrimitiveRenderStateHelper::GetUObject()
{
	return GetEntityManager().GetOwner();
}

const UObject* FMassPrimitiveRenderStateHelper::GetUObject() const
{
	return GetEntityManager().GetOwner();
}

FMatrix FMassPrimitiveRenderStateHelper::GetRenderMatrix() const
{
	return GetTransform().ToMatrixWithScale();
}

float FMassPrimitiveRenderStateHelper::GetLastRenderTimeOnScreen() const
{
	return PrimitiveSceneData.GetLastRenderTimeOnScreen();
}

void FMassPrimitiveRenderStateHelper::SetCollisionEnabled(bool bInCollisionEnabled)
{
	FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	SceneProxyDesc.bCollisionEnabled = bInCollisionEnabled;
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void FMassPrimitiveRenderStateHelper::OnPrecacheFinished(int32 JobSetThatJustCompleted)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_PSOPrecacheFinishedTask);
	int32 CurrJobSetCompleted = LatestPSOPrecacheJobSetCompleted.load();
	while (CurrJobSetCompleted < JobSetThatJustCompleted && !LatestPSOPrecacheJobSetCompleted.compare_exchange_weak(CurrJobSetCompleted, JobSetThatJustCompleted)) {}
	MarkRenderStateDirty();
}

void FMassPrimitiveRenderStateHelper::RequestRecreateRenderStateWhenPSOPrecacheFinished(const FGraphEventArray& PSOPrecacheCompileEvents)
{
	// If the proxy creation strategy relies on knowing when the precached PSO has been compiled,
	// schedule a task to mark the render state dirty when all PSOs are compiled so the proxy gets recreated.
	if (GetPSOPrecacheProxyCreationStrategy() != EPSOPrecacheProxyCreationStrategy::AlwaysCreate)
	{
		LatestPSOPrecacheJobSet++;
		MarkRenderStateDirty();
	}
	bPSOPrecacheCalled = true;
}

void FMassPrimitiveRenderStateHelper::SetupPrecachePSOParams(FPSOPrecacheParams& Params)
{
	const FPrimitiveSceneProxyDesc& SceneProxyDesc = GetSceneProxyDesc();
	//auto IsPrecomputedLightingValid = []() { return false; };
	Params.bRenderInMainPass = SceneProxyDesc.bRenderInMainPass;
	Params.bRenderInDepthPass = SceneProxyDesc.bRenderInDepthPass;
	Params.bStaticLighting = SceneProxyDesc.bHasStaticLighting;
	Params.bUsesIndirectLightingCache = Params.bStaticLighting && SceneProxyDesc.IndirectLightingCacheQuality != ILCQ_Off;// && (!IsPrecomputedLightingValid() || SceneProxyDesc.LightmapType == ELightmapType::ForceVolumetric);
	Params.bAffectDynamicIndirectLighting = SceneProxyDesc.bAffectDynamicIndirectLighting;
	Params.bCastShadow = SceneProxyDesc.CastShadow;
	// Custom depth can be toggled at runtime with PSO precache call so assume it might be needed when depth pass is needed
	// Ideally precache those with lower priority and don't wait on these (UE-174426)
	Params.bRenderCustomDepth = SceneProxyDesc.bRenderInDepthPass;
	Params.bCastShadowAsTwoSided = SceneProxyDesc.bCastShadowAsTwoSided;
	Params.SetMobility(SceneProxyDesc.Mobility);
	Params.SetStencilWriteMask(FRendererStencilMaskEvaluation::ToStencilMask(SceneProxyDesc.CustomDepthStencilWriteMask));

	TArray<UMaterialInterface*> UsedMaterials;
	GetUsedMaterials(UsedMaterials);
	for (const UMaterialInterface* MaterialInterface : UsedMaterials)
	{
		if (MaterialInterface && MaterialInterface->IsUsingWorldPositionOffset_Concurrent(GMaxRHIShaderPlatform))
		{
			Params.bAnyMaterialHasWorldPositionOffset = true;
			break;
		}
	}
}

#endif

bool FMassPrimitiveRenderStateHelper::IsPSOPrecaching() const
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	// Consider as precaching when marked as required to do PSOs precaching (even if task has not been launched yet) 
	return bPSOPrecacheRequired || (LatestPSOPrecacheJobSetCompleted != LatestPSOPrecacheJobSet);
#else
	return false;
#endif
}

bool FMassPrimitiveRenderStateHelper::UsePSOPrecacheFallbackMaterial() const
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	return IsPSOPrecaching() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::UseFallbackMaterialUntilPSOPrecached;
#else
	return false;
#endif
}

bool FMassPrimitiveRenderStateHelper::CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority NewPSOPrecachePriority)
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	bool bPrecacheStillRunning = IsPSOPrecaching();

	ensure(!IsComponentPSOPrecachingEnabled() || bPSOPrecacheCalled || bPSOPrecacheRequired);
	check(NewPSOPrecachePriority == EPSOPrecachePriority::High || NewPSOPrecachePriority == EPSOPrecachePriority::Highest);

	if (bPrecacheStillRunning && PSOPrecacheRequestPriority < NewPSOPrecachePriority)
	{
		// Only boost PSO priority if PSO task was started
		if (LatestPSOPrecacheJobSetCompleted != LatestPSOPrecacheJobSet)
		{
			BoostPSOPriority(NewPSOPrecachePriority, MaterialPSOPrecacheRequestIDs);
		}
		PSOPrecacheRequestPriority = NewPSOPrecachePriority;
	}
	return bPrecacheStillRunning;
#else
	return false;
#endif
}


void FMassPrimitiveRenderStateHelper::MarkPrecachePSOsRequired()
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	if (bPSOPrecacheCalled || !FApp::CanEverRender() || !IsComponentPSOPrecachingEnabled())
	{
		return;
	}
	bPSOPrecacheRequired = true;
	PSOPrecacheRequestPriority = EPSOPrecachePriority::Medium;
#endif
}

void FMassPrimitiveRenderStateHelper::PrecachePSOs()
{
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	TRACE_CPUPROFILER_EVENT_SCOPE(FMassPrimitiveRenderStateHelper::PrecachePSOs);

	if (!bPSOPrecacheRequired)
	{
		return;
	}

	// Clear the current request data
	MaterialPSOPrecacheRequestIDs.Empty();

	// Collect the data from the derived classes
	FPSOPrecacheParams PSOPrecacheParams;
	SetupPrecachePSOParams(PSOPrecacheParams);
	FMaterialInterfacePSOPrecacheParamsList PSOPrecacheDataArray;
	CollectPSOPrecacheData(PSOPrecacheParams, PSOPrecacheDataArray);
	// Set priority
	for (FMaterialInterfacePSOPrecacheParams& Params : PSOPrecacheDataArray)
	{
		Params.Priority = PSOPrecacheRequestPriority;
	}

	FGraphEventArray GraphEvents;
	PrecacheMaterialPSOs(PSOPrecacheDataArray, MaterialPSOPrecacheRequestIDs, GraphEvents);

	RequestRecreateRenderStateWhenPSOPrecacheFinished(GraphEvents);
	bPSOPrecacheRequired = false;
#endif
}

UObject* FMassPrimitiveRenderStateHelper::GetOwner() const
{
	return GetEntityManager().GetOwner();
}

FString FMassPrimitiveRenderStateHelper::GetOwnerName() const
{
	return GetNameSafe(GetEntityManager().GetOwner());
}

FPrimitiveSceneProxy* FMassPrimitiveRenderStateHelper::CreateSceneProxy()
{
	return CreateSceneProxy(/*OutError*/nullptr);
}

//
// Functions below here added for use in streamer, which is not used by FastGeo. These functions may not get used.
static FRenderAssetOwnerStreamingState GFastGeoStreamingStateStub;
FRenderAssetOwnerStreamingState& FMassPrimitiveRenderStateHelper::GetStreamingState() const
{
	return GFastGeoStreamingStateStub;
}

ULevel* FMassPrimitiveRenderStateHelper::GetComponentLevel() const
{
	return nullptr;
}

IPrimitiveComponent* FMassPrimitiveRenderStateHelper::GetLODParentPrimitive() const
{
	return nullptr;
}

float FMassPrimitiveRenderStateHelper::GetMinDrawDistance() const
{
	return 0.0f;
}

float FMassPrimitiveRenderStateHelper::GetStreamingScale() const
{
	return 1.0f;
}

void FMassPrimitiveRenderStateHelper::OnRenderAssetFirstLodChange(const UStreamableRenderAsset* RenderAsset, int32 FirstLodIndex)
{
}

UStreamableRenderAsset* FMassPrimitiveRenderStateHelper::GetStreamableNaniteAsset() const
{
	return nullptr;
}

void FMassPrimitiveRenderStateHelper::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
}

#if WITH_EDITOR
HHitProxy* FMassPrimitiveRenderStateHelper::CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex)
{
	if (FMassTypeElementFragment* TypeElementFragment = GetEntityManager().GetFragmentDataPtr<FMassTypeElementFragment>(EntityHandle))
	{
		return new HTypeElementHandleHitProxy(TypeElementFragment->TypeElementHandle, MaterialIndex, SectionIndex);
	}
	return nullptr;
}
#endif // WITH_EDITOR

HHitProxy* FMassPrimitiveRenderStateHelper::CreatePrimitiveHitProxies(TArray<TRefCountPtr<HHitProxy>>& OutHitProxies)
{
#if WITH_EDITOR
	if (FMassTypeElementFragment* TypeElementFragment = GetEntityManager().GetFragmentDataPtr<FMassTypeElementFragment>(EntityHandle))
	{
		HTypeElementHandleHitProxy* HitProxy = new HTypeElementHandleHitProxy(TypeElementFragment->TypeElementHandle);
		OutHitProxies.Add(HitProxy);
		return HitProxy;
	}
#endif // WITH_EDITOR
	return nullptr;
}

FPrimitiveMaterialPropertyDescriptor FMassPrimitiveRenderStateHelper::GetUsedMaterialPropertyDesc(EShaderPlatform InShaderPlatform) const
{
	return FPrimitiveComponentHelper::GetUsedMaterialPropertyDesc(*this, InShaderPlatform);
}