// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassRenderStateHelper.h"

#include "Components/ComponentInterfaces.h"
#include "Components/SceneComponent.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Engine/EngineTypes.h"
#include "HitProxies.h"
#include "PipelineStateCache.h"
#include "PrimitiveSceneDesc.h"
#include "PrimitiveSceneInfoData.h"
#include "SceneTypes.h"

struct FPrimitiveSceneProxyDesc;
struct FPrimitiveSceneDesc;
struct FMassRenderPrimitiveFragment;
class URuntimeVirtualTexture;
enum EShaderPlatform : uint16;

/**
 * Helper to handle all the communication to the renderer from Mass for all type of primitives
 */
struct FMassPrimitiveRenderStateHelper : public FMassRenderStateHelper, public IPrimitiveComponent
{
public:
	using Super = FMassRenderStateHelper;

	FMassPrimitiveRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager, const FMassRenderPrimitiveFragment& RenderPrimitiveFragment);
	virtual ~FMassPrimitiveRenderStateHelper() override = default;
	
	TConstArrayView<URuntimeVirtualTexture*> GetRuntimeVirtualTextures() const;
	bool IsFirstPersonRelevant() const;

	FPrimitiveComponentId GetPrimitiveSceneId() const;
	virtual UObject const* AdditionalStatObject() const 
	{ 
		return nullptr; 
	}
		
	FMatrix GetRenderMatrix() const;

	// Used by FStaticMeshComponentHelper/FInstancedStaticMeshComponentHelper
	FTransform GetComponentTransform() const { return GetTransform(); }

	// Primitive Fragments helpers
	const FMassRenderPrimitiveFragment& GetRenderPrimitiveFragment() const;
	FMassRenderPrimitiveFragment& GetMutableRenderPrimitiveFragment();

#if WITH_EDITOR
	virtual void NotifyRenderStateChanged();
	void UpdateSelection();
	void UpdateLevelInstanceEditingState();
#endif // WITH_EDITOR
	bool IsDrawnInGame() const;
	bool CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority NewPSOPrecachePriority = EPSOPrecachePriority::High);
	bool IsPSOPrecaching() const;
	void SetCollisionEnabled(bool bInCollisionEnabled);
	void UpdateVisibility();
	void UpdateTransform();

	EComponentMobility::Type GetMobility() const;

	//~ Begin IPrimitiveComponent interface
	virtual bool IsRenderStateDirty() const override;
	virtual bool IsRenderStateCreated() const override;
	virtual bool ShouldCreateRenderState() const override;
	virtual bool IsRegistered() const override;
	virtual bool IsUnreachable() const override;
	virtual bool IsStaticMobility() const override;
	virtual bool IsMipStreamingForced() const override;
	virtual UWorld* GetWorld() const override;
	virtual FSceneInterface* GetScene() const override;
	virtual FPrimitiveSceneProxy* GetSceneProxy() const override;
	virtual void MarkRenderStateDirty() override;
	virtual void DestroyRenderState() override;
	virtual void CreateRenderState(FRegisterComponentContext* Context) override;
	virtual FString GetName() const override;
	virtual FString GetFullName() const override;
	virtual FTransform GetTransform() const override;
	virtual const FBoxSphereBounds& GetBounds() const override;
	virtual float GetLastRenderTimeOnScreen() const override;
	virtual void GetPrimitiveStats(FPrimitiveStats& PrimitiveStats) const override;
	virtual UObject* GetUObject() override;
	virtual const UObject* GetUObject() const override;
	virtual void PrecachePSOs() override;
	virtual UObject* GetOwner() const override;
	virtual FString GetOwnerName() const override;
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual FRenderAssetOwnerStreamingState& GetStreamingState() const override;
	virtual ULevel* GetComponentLevel() const override;
	virtual IPrimitiveComponent* GetLODParentPrimitive() const override;
	virtual float GetMinDrawDistance() const override;
	virtual float GetStreamingScale() const override;
	virtual void OnRenderAssetFirstLodChange(const UStreamableRenderAsset* RenderAsset, int32 FirstLodIndex) override;
	virtual UStreamableRenderAsset* GetStreamableNaniteAsset() const override;
	virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
#if WITH_EDITOR
	virtual HHitProxy* CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) override;
#endif // WITH_EDITOR
	virtual HHitProxy* CreatePrimitiveHitProxies(TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
	//~ End IPrimitiveComponent interface

	//~ Begin FMassRenderStateHelper interface
	virtual void DestroyRenderState(FMassDestroyRenderStateContext* Context) override;
	virtual void MarkPrecachePSOsRequired() override;
	//~End FMassRenderStateHelper interface

	//~ Begin Materials
	virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const = 0;
	virtual int32 GetNumMaterials() const = 0;
	FPrimitiveMaterialPropertyDescriptor GetUsedMaterialPropertyDesc(EShaderPlatform InShaderPlatform) const;
	//~ End Materials
	
protected:
	virtual void InitializeSceneProxyDescDynamicProperties();
	virtual void ResetSceneProxyDescUnsupportedProperties();
	virtual FPrimitiveSceneProxyDesc& GetSceneProxyDesc() = 0;
	virtual const FPrimitiveSceneProxyDesc& GetSceneProxyDesc() const = 0;
	virtual FPrimitiveSceneProxy* CreateSceneProxy(ESceneProxyCreationError* OutError) = 0;
	FPrimitiveSceneDesc BuildSceneDesc();

	friend class UMassCreateRenderSceneProxyProcessor;
private:
	bool UsePSOPrecacheFallbackMaterial() const;

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
public:
	virtual void OnPrecacheFinished(int32 JobSetThatJustCompleted) override;

protected:
	void RequestRecreateRenderStateWhenPSOPrecacheFinished(const FGraphEventArray& PSOPrecacheCompileEvents);
	void SetupPrecachePSOParams(FPSOPrecacheParams& Params);
	virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) {}

	// Cached array of material PSO requests which can be used to boost the priority
	TArray<FMaterialPSOPrecacheRequestID> MaterialPSOPrecacheRequestIDs;
	// Atomic int used to track the last PSO precache events
	std::atomic<int> LatestPSOPrecacheJobSetCompleted = 0;
	int32 LatestPSOPrecacheJobSet = 0;
	// Helper flag to check if PSOs have been precached already
	std::atomic<bool> bPSOPrecacheCalled = false;
	std::atomic<bool> bPSOPrecacheRequired = false;
	// PSOs requested priority
	std::atomic<EPSOPrecachePriority> PSOPrecacheRequestPriority = EPSOPrecachePriority::Medium;
	static_assert((int)EPSOPrecachePriority::Highest < 1 << 2);
#endif

protected:
	// Copied data from fragment as scene proxy decs take a view on this array
	FCustomPrimitiveData CustomPrimitiveData;

	// Runtime data
	FPrimitiveSceneDesc SceneDesc;
	FPrimitiveSceneInfoData PrimitiveSceneData{};
};
