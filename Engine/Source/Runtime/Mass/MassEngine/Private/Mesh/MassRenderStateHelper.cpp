// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRenderStateHelper.h"

#include "MassEngineTypes.h"
#include "Engine/World.h"
#include "MassEntityManager.h"
#include "MassSignalSubsystem.h"
#include "Mesh/MassEngineMeshFragments.h"

// ----------------------------------------------------------------------//
// FMassRenderStateFragment
//----------------------------------------------------------------------//
void FMassRenderStateFragment::DestroyRenderStateHelper()
{
	RenderStateHelper = nullptr;
}

// ----------------------------------------------------------------------//
// FMassRenderStateHelper
//----------------------------------------------------------------------//
FMassRenderStateHelper::FMassRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager)
	: EntityHandle(InEntityHandle)
	, EntityManager(InEntityManager->AsShared())
{
}

UWorld* FMassRenderStateHelper::GetWorld() const
{
	return GetEntityManager().GetWorld();
}

FSceneInterface* FMassRenderStateHelper::GetScene() const
{
	if (UWorld* World = GetWorld())
	{
		return World->Scene;
	}
	return nullptr;
}

FMassEntityManager& FMassRenderStateHelper::GetEntityManager() const
{
	return EntityManager.Get();
}

const FMassRenderStateFragment& FMassRenderStateHelper::GetRenderStateFragment() const
{
	return GetEntityManager().GetFragmentDataChecked<FMassRenderStateFragment>(EntityHandle);
}

FMassRenderStateFragment& FMassRenderStateHelper::GetMutableRenderStateFragment()
{
	return GetEntityManager().GetFragmentDataChecked<FMassRenderStateFragment>(EntityHandle);
}

bool FMassRenderStateHelper::IsRenderStateCreated() const
{
	return ProxyState == EProxyCreationState::Created;
}

bool FMassRenderStateHelper::ShouldCreateRenderState() const 
{ 
	return false; 
}

void FMassRenderStateHelper::CreateRenderState(FRegisterComponentContext* Context)
{
}

void FMassRenderStateHelper::DestroyRenderState(FMassDestroyRenderStateContext* Context) 
{
}

void FMassRenderStateHelper::UpdateVisibility() 
{
}

void FMassRenderStateHelper::MarkPrecachePSOsRequired() 
{
}

void FMassRenderStateHelper::PrecachePSOs() 
{
}

#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void FMassRenderStateHelper::OnPrecacheFinished(int32 JobSetThatJustCompleted) 
{
}
#endif // !WITH_EDITOR && UE_WITH_PSO_PRECACHING

bool FMassRenderStateHelper::IsRenderStateDelayed() const
{
	return ProxyState == EProxyCreationState::Delayed;
}

bool FMassRenderStateHelper::IsRenderStateDirty() const
{
	return bRenderStateDirty;
}

void FMassRenderStateHelper::MarkRenderStateDirty()
{
	if ((IsRenderStateCreated() || IsRenderStateDelayed()) && !bRenderStateDirty)
	{
		bRenderStateDirty = true;

		if (UMassSignalSubsystem* SignalSubsystem = UWorld::GetSubsystem<UMassSignalSubsystem>(GetWorld()))
		{
			SignalSubsystem->SignalEntity(UE::Mass::Signals::RenderStateDirty, EntityHandle);
		}
	}
}