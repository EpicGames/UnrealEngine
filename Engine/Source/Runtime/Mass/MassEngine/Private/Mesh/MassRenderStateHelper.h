// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityElementTypes.h"
#include "Mass/EntityHandle.h"
#include "PSOPrecacheValidation.h"
#include "Mass/ExternalSubsystemTraits.h"

#include "MassRenderStateHelper.generated.h"

struct FMassRenderStateFragment;
struct FMassEntityManager;
class UActorComponent;
class FSceneInterface;
struct FMassRenderStateHelper;
class FRegisterComponentContext;
struct FMassDestroyRenderStateContext;

enum class ESceneProxyCreationError
{
	None,
	WaitingPSOs,
	InvalidMesh
};

/**
 * Fragment holding the render state
 */
USTRUCT()
struct FMassRenderStateFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassRenderStateFragment() = default;
	void DestroyRenderStateHelper();

	template<typename T = FMassRenderStateHelper>
	const T& GetRenderStateHelper() const requires TIsDerivedFrom<T, FMassRenderStateHelper>::Value
	{
		checkf(RenderStateHelper, TEXT("Expecting the ptr to be set"));
		// @Todo find a better way of doing this unsafe cast, Instanced struct are not possible du to atomics member variables
		return static_cast<T&>(*RenderStateHelper);
	}

	template<typename T = FMassRenderStateHelper>
	T& GetRenderStateHelper() requires TIsDerivedFrom<T, FMassRenderStateHelper>::Value
	{
		checkf(RenderStateHelper, TEXT("Expecting the ptr to be set"));
		// @Todo find a better way of doing this unsafe cast, Instanced struct are not possible du to atomics member variables
		return static_cast<T&>(*RenderStateHelper);
	}

	template<typename T, typename... TArgs>
	T* CreateRenderStateHelper(TArgs&&... InArgs) requires TIsDerivedFrom<T, FMassRenderStateHelper>::Value
	{
		checkf(!RenderStateHelper, TEXT("Render State Helper already exist"));
		TSharedPtr<T> CreatedRenderState = MakeShared<T>(Forward<TArgs>(InArgs)...);
		RenderStateHelper = CreatedRenderState;
		return CreatedRenderState.Get();
	}

private:
	TSharedPtr<FMassRenderStateHelper> RenderStateHelper = nullptr;

};

template<>
struct TMassFragmentTraits<FMassRenderStateFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

/**
 * Base helper to handle all the communication to the renderer from Mass
 */
struct FMassRenderStateHelper
{
public:

	FMassRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager);
	virtual ~FMassRenderStateHelper() = default;

	void MarkRenderStateDirty();
	bool IsRenderStateDirty() const;
	bool IsRenderStateDelayed() const;
	bool IsRenderStateCreated() const;
	virtual bool ShouldCreateRenderState() const;
	virtual void CreateRenderState(FRegisterComponentContext* Context);
	virtual void DestroyRenderState(FMassDestroyRenderStateContext* Context);
	virtual void UpdateVisibility();
	virtual void MarkPrecachePSOsRequired();
	virtual void PrecachePSOs();
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual void OnPrecacheFinished(int32 JobSetThatJustCompleted);
#endif

	UWorld* GetWorld() const;
	FSceneInterface* GetScene() const;

	FMassEntityManager& GetEntityManager() const;

protected:
	FMassEntityHandle EntityHandle;
	TSharedRef<FMassEntityManager> EntityManager;

	const FMassRenderStateFragment& GetRenderStateFragment() const;
	FMassRenderStateFragment& GetMutableRenderStateFragment();

	enum class EProxyCreationState : uint8
	{
		None, // Constructed/Initialized
		Creating, // Actively creating the proxy
		Created, // Proxy is now created
		Delayed // Proxy creation delayed (used when PSO precaching is not ready when creating proxy)
	};

	EProxyCreationState ProxyState = EProxyCreationState::None;
	bool bRenderStateDirty = false;
};