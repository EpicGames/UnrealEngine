// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStaticMeshRenderStateHelper.h"

#include "Engine/StaticMesh.h"
#include "MassEntityManager.h"
#include "Mesh/MassEngineMeshFragments.h"
#include "Physics/MassEnginePhysicsFragments.h"
#include "Rendering/NaniteResourcesHelper.h"
#include "StaticMeshComponentHelper.h"
#include "StaticMeshResources.h"
#include "StaticMeshSceneProxyDesc.h"

#if WITH_EDITOR
#include "ObjectCacheEventSink.h"
#include "WorldPartition/HLOD/HLODActor.h"
#endif

// ----------------------------------------------------------------------//
// FMassBaseStaticMeshRenderStateHelper
//----------------------------------------------------------------------//
FMassBaseStaticMeshRenderStateHelper::FMassBaseStaticMeshRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager, const FMassRenderPrimitiveFragment& RenderPrimitiveFragment, const FMassOverrideMaterialsFragment* OverrideMaterialsFragment)
	: Super(InEntityHandle, InEntityManager, RenderPrimitiveFragment, OverrideMaterialsFragment)
{
}

const Nanite::FResources* FMassBaseStaticMeshRenderStateHelper::GetNaniteResources() const
{
	if (GetStaticMesh() && GetStaticMesh()->GetRenderData())
	{
		return GetStaticMesh()->GetRenderData()->NaniteResourcesPtr.Get();
	}
	return nullptr;
}

#if WITH_EDITOR
void FMassBaseStaticMeshRenderStateHelper::NotifyRenderStateChanged()
{
	Super::NotifyRenderStateChanged();

	FObjectCacheEventSink::NotifyStaticMeshChanged_Concurrent(this);
}
#endif

int32 FMassBaseStaticMeshRenderStateHelper::GetNumMaterials() const
{
	return GetStaticMesh() ? GetStaticMesh()->GetStaticMaterials().Num() : 0;
}

UMaterialInterface* FMassBaseStaticMeshRenderStateHelper::GetMaterial(int32 MaterialIndex) const
{
	return GetMaterial(MaterialIndex, false);
}

const FCollisionResponseContainer& FMassBaseStaticMeshRenderStateHelper::GetCollisionResponseToChannels() const
{
	if (FMassPhysicsCollisionSettingsFragment* CollisionSettingsFragment = GetEntityManager().GetConstSharedFragmentDataPtr<FMassPhysicsCollisionSettingsFragment>(EntityHandle))
	{
		return CollisionSettingsFragment->CollisionResponse;
	}

	return FCollisionResponseContainer::GetDefaultResponseContainer();
}

UMaterialInterface* FMassBaseStaticMeshRenderStateHelper::GetNaniteAuditMaterial(int32 MaterialIndex) const
{
	return GetMaterial(MaterialIndex, true);
}

bool FMassBaseStaticMeshRenderStateHelper::UseNaniteOverrideMaterials(bool bDoingMaterialAudit) const
{
	return Nanite::FNaniteResourcesHelper::UseNaniteOverrideMaterials(*this, bDoingMaterialAudit);
}

bool FMassBaseStaticMeshRenderStateHelper::ShouldCreateNaniteProxy(Nanite::FMaterialAudit* OutNaniteMaterials) const
{
	return Nanite::FNaniteResourcesHelper::ShouldCreateNaniteProxy(GetStaticMeshSceneProxyDesc(), OutNaniteMaterials);
}

UMaterialInterface* FMassBaseStaticMeshRenderStateHelper::GetMaterial(int32 MaterialIndex, bool bDoingNaniteMaterialAudit) const
{
	return FStaticMeshComponentHelper::GetMaterial(*this, MaterialIndex, bDoingNaniteMaterialAudit);
}

bool FMassBaseStaticMeshRenderStateHelper::HasValidNaniteData() const
{
	return Nanite::FNaniteResourcesHelper::HasValidNaniteData(*this);
}

void FMassBaseStaticMeshRenderStateHelper::ResetSceneProxyDescUnsupportedProperties()
{
	Super::ResetSceneProxyDescUnsupportedProperties();

	// Unsupported properties
	FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
	SceneProxyDesc.LODData = TArrayView<struct FStaticMeshComponentLODInfo>();
	SceneProxyDesc.LODParentPrimitive = nullptr;
	SceneProxyDesc.MeshPaintTexture = nullptr;
	SceneProxyDesc.MeshPaintTextureCoordinateIndex = 0;
#if STATICMESH_ENABLE_DEBUG_RENDERING
	SceneProxyDesc.bDrawMeshCollisionIfComplex = false;
	SceneProxyDesc.bDrawMeshCollisionIfSimple = false;
#endif // STATICMESH_ENABLE_DEBUG_RENDERING
	SceneProxyDesc.bDisplayNaniteFallbackMesh = false;
#if WITH_EDITORONLY_DATA
	SceneProxyDesc.SectionIndexPreview = INDEX_NONE;
	SceneProxyDesc.MaterialIndexPreview = INDEX_NONE;
	SceneProxyDesc.SelectedEditorMaterial = INDEX_NONE;
	SceneProxyDesc.SelectedEditorSection = INDEX_NONE;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
	SceneProxyDesc.MaterialStreamingRelativeBoxes = TArrayView<uint32>();
#endif // WITH_EDITORONLY_DATA

	// Properties that will be initialized by InitializeSceneProxyDescDynamicProperties
	SceneProxyDesc.NaniteResources = nullptr;
	SceneProxyDesc.BodySetup = nullptr;
	SceneProxyDesc.MaterialRelevance = FMaterialRelevance();
	SceneProxyDesc.bUseProvidedMaterialRelevance = false;
}

void FMassBaseStaticMeshRenderStateHelper::InitializeSceneProxyDescDynamicProperties()
{
	Super::InitializeSceneProxyDescDynamicProperties();

	FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
	TConstArrayView<TObjectPtr<UMaterialInterface>> OverrideMaterials = GetOverrideMaterials();
	SceneProxyDesc.OverrideMaterials = (TArrayView<TObjectPtr<UMaterialInterface>>&)OverrideMaterials; // ugly cast, but just making it work for now. It would mean to change the SceneProxyDesc to fix this.
	SceneProxyDesc.NaniteResources = GetNaniteResources();
	SceneProxyDesc.SetMaterialRelevance(GetMaterialRelevance(GetScene()->GetShaderPlatform()));
	SceneProxyDesc.SetCollisionResponseToChannels(GetCollisionResponseToChannels());

	// Add LODData support
	// Add LODParentPrimitive support
	// Add MeshPaintTexture/MeshPaintTextureCoordinateIndex support
}

FPrimitiveSceneProxy* FMassBaseStaticMeshRenderStateHelper::CreateSceneProxy(ESceneProxyCreationError* OutError)
{
	check(GetWorld());
	FSceneInterface* Scene = GetScene();
	UStaticMesh* StaticMesh = GetStaticMesh();
	check(Scene);
	check(StaticMesh);
	check(StaticMesh->GetRenderData());
	check(StaticMesh->GetRenderData()->IsInitialized());
	check(!StaticMesh->IsCompiling());

	if (OutError)
	{
		*OutError = ESceneProxyCreationError::None;
	}

	InitializeSceneProxyDescDynamicProperties();

	const FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
	check(SceneProxyDesc.Scene);
	check(SceneProxyDesc.Scene == Scene);
	check(SceneProxyDesc.World == Scene->GetWorld());
	check(SceneProxyDesc.FeatureLevel == Scene->GetFeatureLevel());
	check(SceneProxyDesc.ComponentId == GetPrimitiveSceneId());

	FStaticMeshComponentHelper::ESceneProxyCreationError SceneProxyCreationError;
	FPrimitiveSceneProxy* SceneProxy = FStaticMeshComponentHelper::CreateSceneProxy<FMassBaseStaticMeshRenderStateHelper, /*bRenderDataReady=*/true>(*this, &SceneProxyCreationError);

	if (SceneProxy == nullptr && OutError)
	{
		switch (SceneProxyCreationError)
		{
		case FStaticMeshComponentHelper::ESceneProxyCreationError::WaitingPSOs:
			*OutError = ESceneProxyCreationError::WaitingPSOs;
			break;

		default:
			*OutError = ESceneProxyCreationError::InvalidMesh;
			break;
		}
	}

	return SceneProxy;
}


FPrimitiveSceneProxy* FMassBaseStaticMeshRenderStateHelper::CreateStaticMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite)
{
	check(GetWorld());
	const FStaticMeshSceneProxyDesc& SceneProxyDesc = GetStaticMeshSceneProxyDesc();
	check(SceneProxyDesc.Scene);

	if (bCreateNanite)
	{
		PrimitiveSceneData.SceneProxy = ::new Nanite::FSceneProxy(NaniteMaterials, SceneProxyDesc);
	}
	else
	{
		PrimitiveSceneData.SceneProxy = ::new FStaticMeshSceneProxy(SceneProxyDesc, false);
	}
	return PrimitiveSceneData.SceneProxy;
}

void FMassBaseStaticMeshRenderStateHelper::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	FStaticMeshComponentHelper::GetUsedMaterials(*this, OutMaterials, bGetDebugMaterials);
}

void FMassBaseStaticMeshRenderStateHelper::GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const
{
	OutMaterialSlotOverlayMaterials.Reset();
	if (GetStaticMesh())
	{
		const TArray<FStaticMaterial>& StaticMaterials = GetStaticMesh()->GetStaticMaterials();
		for (const FStaticMaterial& StaticMaterial : StaticMaterials)
		{
			OutMaterialSlotOverlayMaterials.Add(StaticMaterial.OverlayMaterialInterface);
		}
	}
}

const TArray<TObjectPtr<UMaterialInterface>>& FMassBaseStaticMeshRenderStateHelper::GetComponentMaterialSlotsOverlayMaterial() const
{
	return GetStaticMeshSceneProxyDesc().MaterialSlotsOverlayMaterial;
}

#if WITH_EDITOR
void FMassBaseStaticMeshRenderStateHelper::PostStaticMeshCompilation()
{
	MarkRenderStateDirty();
}
#endif // WITH_EDITOR

UStaticMesh* FMassBaseStaticMeshRenderStateHelper::GetStaticMesh() const
{
	return GetStaticMeshSceneProxyDesc().StaticMesh;
}

IPrimitiveComponent* FMassBaseStaticMeshRenderStateHelper::GetPrimitiveComponentInterface()
{
	return this;
}

UObject const* FMassBaseStaticMeshRenderStateHelper::AdditionalStatObject() const
{
	return GetStaticMesh();
}

UMaterialInterface* FMassBaseStaticMeshRenderStateHelper::GetOverlayMaterial() const
{
	return GetStaticMeshSceneProxyDesc().OverlayMaterial;
}

bool FMassBaseStaticMeshRenderStateHelper::IsReverseCulling() const
{
	return GetStaticMeshSceneProxyDesc().IsReverseCulling();
}

bool FMassBaseStaticMeshRenderStateHelper::IsDisallowNanite() const
{
	return GetStaticMeshSceneProxyDesc().IsDisallowNanite();
}

bool FMassBaseStaticMeshRenderStateHelper::IsForceDisableNanite() const
{
	return GetStaticMeshSceneProxyDesc().IsForceDisableNanite();
}

bool FMassBaseStaticMeshRenderStateHelper::IsForceNaniteForMasked() const
{
	return GetStaticMeshSceneProxyDesc().IsForceNaniteForMasked();
}

int32 FMassBaseStaticMeshRenderStateHelper::GetForcedLodModel() const
{
	return GetStaticMeshSceneProxyDesc().GetForcedLodModel();
}

bool FMassBaseStaticMeshRenderStateHelper::GetOverrideMinLOD() const
{
	return GetStaticMeshSceneProxyDesc().bOverrideMinLOD;
}

int32 FMassBaseStaticMeshRenderStateHelper::GetMinLOD() const
{
	return GetStaticMeshSceneProxyDesc().MinLOD;
}

bool FMassBaseStaticMeshRenderStateHelper::GetForceNaniteForMasked() const
{
	return GetStaticMeshSceneProxyDesc().bForceNaniteForMasked;
}

int32 FMassBaseStaticMeshRenderStateHelper::GetWorldPositionOffsetDisableDistance() const
{
	return GetStaticMeshSceneProxyDesc().GetWorldPositionOffsetDisableDistance();
}

float FMassBaseStaticMeshRenderStateHelper::GetNanitePixelProgrammableDistance() const
{
	return GetStaticMeshSceneProxyDesc().GetNanitePixelProgrammableDistance();
}

bool FMassBaseStaticMeshRenderStateHelper::EvaluateWorldPositionOffsetInRayTracing() const
{
	return GetStaticMeshSceneProxyDesc().EvaluateWorldPositionOffsetInRayTracing();
}

#if WITH_EDITORONLY_DATA
bool FMassBaseStaticMeshRenderStateHelper::IsDisplayNaniteFallbackMesh() const
{
	return GetStaticMeshSceneProxyDesc().IsDisplayNaniteFallbackMesh();
}
#endif


#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
void FMassBaseStaticMeshRenderStateHelper::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	// @TODO Implement this
	//FStaticMeshComponentHelper::CollectPSOPrecacheData(*this, BasePrecachePSOParams, OutParams);
}
#endif

// ----------------------------------------------------------------------//
// FMassStaticMeshRenderStateHelper
//----------------------------------------------------------------------//
FMassStaticMeshRenderStateHelper::FMassStaticMeshRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager, const FMassRenderPrimitiveFragment& RenderPrimitiveFragment, const FMassOverrideMaterialsFragment* OverrideMaterialsFragment, const FMassRenderStaticMeshFragment& RenderStaticMeshFragment)
	: Super(InEntityHandle, InEntityManager, RenderPrimitiveFragment, OverrideMaterialsFragment)
{
}

const FMassRenderStaticMeshFragment& FMassStaticMeshRenderStateHelper::GetRenderStaticMeshFragment() const
{
	return GetEntityManager().GetFragmentDataChecked<FMassRenderStaticMeshFragment>(EntityHandle);
}

FMassRenderStaticMeshFragment& FMassStaticMeshRenderStateHelper::GetMutableRenderStaticMeshFragment()
{
	return GetEntityManager().GetFragmentDataChecked<FMassRenderStaticMeshFragment>(EntityHandle);
}

FPrimitiveSceneProxyDesc& FMassStaticMeshRenderStateHelper::GetSceneProxyDesc()
{
	return GetStaticMeshSceneProxyDesc();
}

const FPrimitiveSceneProxyDesc& FMassStaticMeshRenderStateHelper::GetSceneProxyDesc() const
{
	return GetStaticMeshSceneProxyDesc();
}

FStaticMeshSceneProxyDesc& FMassStaticMeshRenderStateHelper::GetStaticMeshSceneProxyDesc()
{
	FMassRenderStaticMeshFragment& RenderStaticMeshFragment = GetMutableRenderStaticMeshFragment();
	checkf(RenderStaticMeshFragment.StaticMeshSceneProxyDesc, TEXT("Expecting a valid scene proxy desc"));
	return *RenderStaticMeshFragment.StaticMeshSceneProxyDesc;
}

const FStaticMeshSceneProxyDesc& FMassStaticMeshRenderStateHelper::GetStaticMeshSceneProxyDesc() const
{
	const FMassRenderStaticMeshFragment& RenderStaticMeshFragment = GetRenderStaticMeshFragment();
	checkf(RenderStaticMeshFragment.StaticMeshSceneProxyDesc, TEXT("Expecting a valid scene proxy desc"));
	return *RenderStaticMeshFragment.StaticMeshSceneProxyDesc;
}
