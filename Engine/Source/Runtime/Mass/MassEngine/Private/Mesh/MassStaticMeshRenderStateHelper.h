// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassMeshRenderStateHelper.h"

class FColorVertexBuffer;
class UMaterialInterface;
struct FStaticMeshSceneProxyDesc;
struct FMassRenderStaticMeshFragment;
namespace Nanite
{
	struct FResources;
	struct FMaterialAudit;
}

/**
 * Base helper to handle all the communication to the renderer from Mass for all type of static meshes
 */
struct FMassBaseStaticMeshRenderStateHelper : public FMassMeshRenderStateHelper, public IStaticMeshComponent
{
public:
	using Super = FMassMeshRenderStateHelper;

	FMassBaseStaticMeshRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager, const FMassRenderPrimitiveFragment& RenderPrimitiveFragment, const FMassOverrideMaterialsFragment* OverrideMaterialsFragment);
	virtual ~FMassBaseStaticMeshRenderStateHelper() override = default;

	const FCollisionResponseContainer& GetCollisionResponseToChannels() const;
	UMaterialInterface* GetNaniteAuditMaterial(int32 MaterialIndex) const;
	const Nanite::FResources* GetNaniteResources() const;
	bool HasValidNaniteData() const;
	bool UseNaniteOverrideMaterials(bool bDoingMaterialAudit) const;
	bool ShouldCreateNaniteProxy(Nanite::FMaterialAudit* OutNaniteMaterials) const;
	virtual UObject const* AdditionalStatObject() const override;
	bool IsReverseCulling() const;
	bool IsDisallowNanite() const;
	bool IsForceDisableNanite() const;
	bool IsForceNaniteForMasked() const;
	int32 GetForcedLodModel() const;
	bool GetOverrideMinLOD() const;
	int32 GetMinLOD() const;
	bool GetForceNaniteForMasked() const;
	int32 GetWorldPositionOffsetDisableDistance() const;
	float GetNanitePixelProgrammableDistance() const;
	bool EvaluateWorldPositionOffsetInRayTracing() const;
#if WITH_EDITORONLY_DATA
	bool IsDisplayNaniteFallbackMesh() const;
#endif

	//~ Begin IStaticMeshComponent interface
#if WITH_EDITOR
	virtual void OnMeshRebuild(bool bRenderDataChanged) override {}
	virtual void PreStaticMeshCompilation() override {}
	virtual void PostStaticMeshCompilation() override;
#endif
	virtual UStaticMesh* GetStaticMesh() const override;
	virtual IPrimitiveComponent* GetPrimitiveComponentInterface() override;
	//~ End IStaticMeshComponent interface

	//~ Begin FMassPrimitiveRenderStateHelper interface
	virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	virtual int32 GetNumMaterials() const override;
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
#if WITH_EDITOR
	virtual void NotifyRenderStateChanged() override;
#endif
	//~ End FMassPrimitiveRenderStateHelper interface

protected:
	//~ Begin FMassPrimitiveRenderStateHelper interface
	virtual void ResetSceneProxyDescUnsupportedProperties() override;
	virtual void InitializeSceneProxyDescDynamicProperties() override;
	virtual FPrimitiveSceneProxy* CreateSceneProxy(ESceneProxyCreationError* OutError = nullptr) override;
	//~ End FMassPrimitiveRenderStateHelper interface

	//~ Begin FMassMeshRenderStateHelper interface
	virtual UMaterialInterface* GetOverlayMaterial() const override;
	virtual const TArray<TObjectPtr<UMaterialInterface>>& GetComponentMaterialSlotsOverlayMaterial() const override;
	virtual void GetDefaultMaterialSlotsOverlayMaterial(TArray<TObjectPtr<UMaterialInterface>>& OutMaterialSlotOverlayMaterials) const override;
	//~ End FMassMeshRenderStateHelper interface

	virtual FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() = 0;
	virtual const FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() const = 0;
	virtual FPrimitiveSceneProxy* CreateStaticMeshSceneProxy(const Nanite::FMaterialAudit& NaniteMaterials, bool bCreateNanite);
	UMaterialInterface* GetMaterial(int32 MaterialIndex, bool bDoingNaniteMaterialAudit) const;
#if !WITH_EDITOR && UE_WITH_PSO_PRECACHING
	virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams);
#endif

	friend class FMeshComponentHelper;
	friend class FPrimitiveComponentHelper;
	friend class FStaticMeshComponentHelper;
	friend class FInstancedStaticMeshComponentHelper;
};

/**
 * Helper to handle all the communication to the renderer from Mass for all type of static meshes
 */
struct FMassStaticMeshRenderStateHelper : public FMassBaseStaticMeshRenderStateHelper
{
public:
	using Super = FMassBaseStaticMeshRenderStateHelper;

	FMassStaticMeshRenderStateHelper(FMassEntityHandle InEntityHandle, TNotNull<FMassEntityManager*> InEntityManager, const FMassRenderPrimitiveFragment& RenderPrimitiveFragment, const FMassOverrideMaterialsFragment* OverrideMaterialsFragment, const FMassRenderStaticMeshFragment& RenderStaticMeshFragment);
	virtual ~FMassStaticMeshRenderStateHelper() = default;

	// StaticMesh Fragments helpers
	const FMassRenderStaticMeshFragment& GetRenderStaticMeshFragment() const;
	FMassRenderStaticMeshFragment& GetMutableRenderStaticMeshFragment();

protected:
	//~ Begin FMassPrimitiveRenderStateHelper interface
	virtual FPrimitiveSceneProxyDesc& GetSceneProxyDesc() override;
	virtual const FPrimitiveSceneProxyDesc& GetSceneProxyDesc() const override;
	//~ End FMassPrimitiveRenderStateHelper interface

	//~ Begin FMassBaseStaticMeshRenderStateHelper interface
	virtual FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() override;
	virtual const FStaticMeshSceneProxyDesc& GetStaticMeshSceneProxyDesc() const override;
	//~ End FMassBaseStaticMeshRenderStateHelper interface
};