// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassPhysicsProcessors.h"

#include "Chaos/ChaosUserEntity.h"
#include "Mass/EntityFragments.h"
#include "MassEngineTypes.h"
#include "MassExecutionContext.h"
#include "Mesh/MassEngineMeshFragments.h"
#include "Misc/HashBuilder.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/MassEnginePhysicsFragments.h"
#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassPhysicsProcessors)

namespace UE::MassEngine::Private
{
enum class EBodyType : uint8
{
	Static,
	NonSimulatedDynamic,
	SimulatedDynamic
};
}

//----------------------------------------------------------------------//
// UMassPhysicsBodyInstancesBatcher
//----------------------------------------------------------------------//
UMassPhysicsBodyInstancesBatcher::UMassPhysicsBodyInstancesBatcher(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EntityQuery(*this)
{
	bRequiresGameThreadExecution = false;

	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::SetupBatchedInitializations;
}

void UMassPhysicsBodyInstancesBatcher::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddConstSharedRequirement<FMassStaticMeshFragment>();
	EntityQuery.AddConstSharedRequirement<FMassPhysicsCollisionSettingsFragment>();

	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassPhysicsBodyInstanceFragment>(EMassFragmentAccess::ReadWrite);

	// Init
	EntityQuery.AddTagRequirement<FMassPhysicsBodyInstanceInitializedTag>(EMassFragmentPresence::None);

	// Create the batching entity and link it but let the UMassPhysicsBodyInstancesBatchInitializer initialize the instances
	FMassElementBitSet Bitset;
	Bitset.Add<FMassPhysicsBodyInstancesInitializationRequestFragment>();
	BatchInitializationRequestArchetype = EntityManager->CreateArchetype(Bitset, FMassArchetypeCreationParams{ "PhysicsBodyBatchInitializationRequest" });
	ProcessorEntityCreationRequirements.AddCreatedArchetype(Bitset);
}

void UMassPhysicsBodyInstancesBatcher::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	using namespace UE::MassEngine::Private;

	struct FBatchInfo
	{
		const FMassStaticMeshFragment& StaticMeshFragment;
		uint32 SourceSharedFragmentHash = 0;
		uint32 InstanceCount = 0;
		uint32 CurrentInstanceIndex = 0;
		EBodyType BodyType = EBodyType::Static;
	};

	struct FBatches
	{
		const FMassPhysicsCollisionSettingsFragment* CollisionSettingsFragment = nullptr;
		TArray<FBatchInfo> BatchInfos;
		TArray<FMassEntityHandle> CreatedRequestEntities;
	};

	TArray<FBatches> Batches;

	auto RetrieveBatchInfo =
		[&Batches](const FMassStaticMeshFragment& StaticMeshFragment
			, const FMassPhysicsCollisionSettingsFragment* CollisionSettingsFragment
			, const EBodyType BodyType
			, const bool bCreateIfNotFound
			, FMassEntityHandle* OutRequestEntity = nullptr) -> FBatchInfo*
		{
			FBatches* Batch = Batches.FindByPredicate([CollisionSettingsFragment](const FBatches& Batch) { return Batch.CollisionSettingsFragment == CollisionSettingsFragment; });
			if (Batch == nullptr)
			{
				if (!bCreateIfNotFound)
				{
					return nullptr;
				}
				Batch = &Batches.Emplace_GetRef(CollisionSettingsFragment);
			}

			// Build the source shared fragment hash
			FHashBuilder HashBuilder;
			HashBuilder << &StaticMeshFragment << BodyType;

			// Find if the batch info already exists, if not add it
			FBatchInfo* BatchInfo;
			int32 Index = Batch->BatchInfos.IndexOfByPredicate([Hash = HashBuilder.GetHash()](const FBatchInfo& EntityInfo) { return EntityInfo.SourceSharedFragmentHash == Hash; });
			if (Index == INDEX_NONE)
			{
				if (!bCreateIfNotFound)
				{
					return nullptr;
				}
				Index = Batch->BatchInfos.Num();
				BatchInfo = &Batch->BatchInfos.Emplace_GetRef(FBatchInfo{.StaticMeshFragment = StaticMeshFragment, .SourceSharedFragmentHash = HashBuilder.GetHash()});
				BatchInfo->BodyType = BodyType;
			}
			else
			{
				BatchInfo = &Batch->BatchInfos[Index];
			}

			if (OutRequestEntity)
			{
				checkf(Index < Batch->CreatedRequestEntities.Num(), TEXT("Expecting all the entities to be created at this point"));
				*OutRequestEntity = Batch->CreatedRequestEntities[Index];
			}
			return BatchInfo;
		};

	// First, gather all the different request entities we need to create.
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&RetrieveBatchInfo](const FMassExecutionContext& Context)
		{
			const FMassStaticMeshFragment& StaticMeshFragment = Context.GetConstSharedFragment<FMassStaticMeshFragment>();
			const FMassPhysicsCollisionSettingsFragment* ColliderFragment = Context.GetConstSharedFragmentPtr<FMassPhysicsCollisionSettingsFragment>();

			const EBodyType BodyType =
				Context.DoesArchetypeHaveTag<FMassPhysicsSimulatedBodyTag>() ?
				EBodyType::SimulatedDynamic
				: Context.DoesArchetypeHaveTag<FMassPhysicsDynamicBodyTag>() ?
				EBodyType::NonSimulatedDynamic
				: EBodyType::Static;

			// Find the batch matching the shared fragment
			FBatchInfo* BatchInfo = RetrieveBatchInfo(StaticMeshFragment, ColliderFragment, BodyType, /*bCreateIfNotFound*/true);
			checkf(BatchInfo, TEXT("Expecting the batch to be created"));
			BatchInfo->InstanceCount += Context.GetNumEntities();
		});

	// Batch create the mass entities for all batched initialization requests
	for (FBatches& Batch : Batches)
	{
		ExecutionContext.BatchCreateEntities(BatchInitializationRequestArchetype, Batch.BatchInfos.Num(), Batch.CreatedRequestEntities);
	}

	// Gather transforms for each entity and linked them to the created request entity
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[this, &RetrieveBatchInfo](FMassExecutionContext& Context)
		{
			const FMassStaticMeshFragment& StaticMeshFragment = Context.GetConstSharedFragment<FMassStaticMeshFragment>();
			const FMassPhysicsCollisionSettingsFragment* ColliderFragment = Context.GetConstSharedFragmentPtr<FMassPhysicsCollisionSettingsFragment>();
			const TConstArrayView<FTransformFragment> TransformFragments = Context.GetFragmentView<FTransformFragment>();
			const TArrayView<FMassPhysicsBodyInstanceFragment> BodyInstanceFragments = Context.GetMutableFragmentView<FMassPhysicsBodyInstanceFragment>();

			const bool bSimulated = Context.DoesArchetypeHaveTag<FMassPhysicsSimulatedBodyTag>();
			const bool bDynamic = Context.DoesArchetypeHaveTag<FMassPhysicsDynamicBodyTag>();

			const EBodyType BodyType =
				bSimulated ? EBodyType::SimulatedDynamic
				: bDynamic ? EBodyType::NonSimulatedDynamic
				: EBodyType::Static;

			FMassEntityHandle RequestEntityHandle;
			FBatchInfo* BatchInfo = RetrieveBatchInfo(StaticMeshFragment, ColliderFragment, BodyType, /*bCreateIfNotFound*/false, &RequestEntityHandle);
			checkf(BatchInfo, TEXT("Expecting the batch to be created"));
			checkf(BatchInfo->BodyType == BodyType, TEXT("Expecting the batch to be created"));
			uint32& InstanceIndex = BatchInfo->CurrentInstanceIndex;

			const FMassEntityManager& EntityManager = Context.GetEntityManagerChecked();

			FMassPhysicsBodyInstancesInitializationRequestFragment& RequestFragment = EntityManager.GetFragmentDataChecked<FMassPhysicsBodyInstancesInitializationRequestFragment>(RequestEntityHandle);
			RequestFragment.StaticMesh = BatchInfo->StaticMeshFragment.Mesh;
			RequestFragment.Transforms.Reserve(BatchInfo->InstanceCount);
			RequestFragment.InstanceBodies.Reserve(BatchInfo->InstanceCount);
			RequestFragment.ChaosUserDefinedEntity.Reserve(BatchInfo->InstanceCount);
			RequestFragment.bDynamic = BodyType != EBodyType::Static;

			UE_VLOG_UELOG(this, LogMassEngine, Log, TEXT("Created initialization request for %d %s body instances using mesh: %s")
				, BatchInfo->InstanceCount
				, bSimulated ? TEXT("simulated dynamic") : bDynamic ? TEXT("non-simulated dynamic") : TEXT("static")
				, *GetPathNameSafe(BatchInfo->StaticMeshFragment.Mesh.Get()));

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				ensureMsgf(!BodyInstanceFragments[*EntityIt].BodyInstanceOwnerObject.IsExplicitlyNull(), TEXT("BodyInstanceOwnerObject is required to determine the validity of the raw pointers"));
				if (BodyInstanceFragments[*EntityIt].BodyInstanceOwnerObject.IsValid())
				{
					verifyf(RequestFragment.Transforms.Add(TransformFragments[*EntityIt].GetTransform()) == InstanceIndex
						, TEXT("Expecting the added index to match the current instance"));

					// Fragment created from serialized data need to create the body
					if (!BodyInstanceFragments[*EntityIt].BodyInstance.IsValid())
					{
						BodyInstanceFragments[*EntityIt].BodyInstance = MakeShared<FBodyInstance>();

						// @todo: ChaosUserDefinedEntity is currently not required for "mass-only" version
					}
					verifyf(RequestFragment.InstanceBodies.Add(BodyInstanceFragments[*EntityIt].BodyInstance) == InstanceIndex
						, TEXT("Expecting the added index to match the current instance"));

					verifyf(RequestFragment.ChaosUserDefinedEntity.Add(BodyInstanceFragments[*EntityIt].ChaosUserDefinedEntity) == InstanceIndex
						, TEXT("Expecting the added index to match the current instance"));

					ensure(bSimulated == RequestFragment.InstanceBodies[InstanceIndex]->bSimulatePhysics);
					InstanceIndex++;
				}
				else
				{
					UE_LOG(LogMassEngine, Warning, TEXT("Skipped instance: BodyInstanceOwnerObject is no longer valid. Removing stale fragment."));
					Context.Defer().RemoveFragment<FMassPhysicsBodyInstanceFragment>(Context.GetEntity(*EntityIt));
				}

				// Tag unconditionally to exclude from future Batcher runs.
				// For valid entities this marks a successful batch; for stale-owner
				// entities this prevents an infinite retry (the owner is destroyed and IsValid() will never return true again).
				Context.Defer().AddTag<FMassPhysicsBodyInstanceInitializedTag>(Context.GetEntity(*EntityIt));
			}
		});
}

//----------------------------------------------------------------------//
// UMassPhysicsBodyInstancesBatchInitializer
//----------------------------------------------------------------------//
UMassPhysicsBodyInstancesBatchInitializer::UMassPhysicsBodyInstancesBatchInitializer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EntityQuery(*this)
{
	bRequiresGameThreadExecution = !FPhysScene::SupportsAsyncPhysicsStateCreation();

	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);

	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::ProcessBatchedInitializations;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::SetupBatchedInitializations);
}

void UMassPhysicsBodyInstancesBatchInitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassPhysicsBodyInstancesInitializationRequestFragment>(EMassFragmentAccess::ReadWrite);
}

void UMassPhysicsBodyInstancesBatchInitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	using namespace UE::MassEngine::Private;

	//////////////////////////////////////////////////////////////////////////////
	// Batch create the mass entities
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[this](FMassExecutionContext& Context)
		{
			const TArrayView<FMassPhysicsBodyInstancesInitializationRequestFragment> RequestFragments = Context.GetMutableFragmentView<FMassPhysicsBodyInstancesInitializationRequestFragment>();

			TRACE_CPUPROFILER_EVENT_SCOPE(UMassPhysicsBodyInstancesBatchInitializer::InitAllInstanceBodies);
			CSV_SCOPED_TIMING_STAT_EXCLUSIVE(UMassPhysicsBodyInstancesBatchInitializer);

			TArray<FMassEntityHandle> EntitiesToDestroy;
			EntitiesToDestroy.Reserve(Context.GetNumEntities());

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				EntitiesToDestroy.Add(Context.GetEntity(*EntityIt));

				FMassPhysicsBodyInstancesInitializationRequestFragment& RequestFragment = RequestFragments[*EntityIt];
				const int32 NumBodies = RequestFragment.InstanceBodies.Num();
				if (!ensureMsgf(NumBodies > 0
					, TEXT("Initialization request fragment was created with an empty list for body instances."
						" Make sure all code paths properly fill the request.")))
				{
					continue;
				}

				if (!ensureMsgf(RequestFragment.Transforms.Num() == NumBodies
					, TEXT("Initialization request fragment was created with an incorrect number of transforms. Expecting %d but has %d"
						" Make sure all code paths properly fill the request.")
					, NumBodies
					, RequestFragment.Transforms.Num()))
				{
					continue;
				}

				if (!ensureMsgf(!RequestFragment.StaticMesh.IsExplicitlyNull()
					, TEXT("Initialization request fragment was created without a valid mesh."
						" Make sure all code paths properly fill the request.")))
				{
					continue;
				}

				const UStaticMesh* Mesh = RequestFragment.StaticMesh.Get();
				if (Mesh == nullptr)
				{
					UE_VLOG_UELOG(this, LogMassEngine, Log, TEXT("Specified mesh is no longer valid. Ignoring the request"));
					continue;
				}

				UBodySetup* BodySetup = Mesh->GetBodySetup();
				if (BodySetup == nullptr)
				{
					UE_VLOG_UELOG(this, LogMassEngine, Log, TEXT("Invalid BodySetup in the specified mesh. Ignoring the request"));
					continue;
				}

				if (RequestFragment.BodyInstanceOwner.IsStale())
				{
					UE_VLOG_UELOG(this, LogMassEngine, Log, TEXT("Specified instance's owner is no longer valid. Ignoring the request"));
					continue;
				}

				// Sanitized array does not contain any nulls
				TArray<FBodyInstance*> InstanceBodiesSanitized;
				InstanceBodiesSanitized.Reserve(NumBodies);

				TArray<FTransform> Transforms;
				Transforms.Reserve(NumBodies);

				EBodyType CommonBodyType = RequestFragment.bDynamic ? EBodyType::NonSimulatedDynamic : EBodyType::Static;
				for (int32 BodyIndex = 0; BodyIndex < NumBodies; ++BodyIndex)
				{
					if (const TSharedPtr<FBodyInstance>& Instance = RequestFragment.InstanceBodies[BodyIndex])
					{
						if (Instance->bSimulatePhysics
							&& CommonBodyType == EBodyType::NonSimulatedDynamic)
						{
							CommonBodyType = EBodyType::SimulatedDynamic;
						}
						ensure((CommonBodyType == UE::MassEngine::Private::EBodyType::Static && !Instance->bSimulatePhysics)
							|| (CommonBodyType == UE::MassEngine::Private::EBodyType::SimulatedDynamic && Instance->bSimulatePhysics)
							|| (CommonBodyType == UE::MassEngine::Private::EBodyType::NonSimulatedDynamic && !Instance->bSimulatePhysics));

						// We need clean up existing static body when getting initialized again with different settings
						if (Instance->IsValidBodyInstance())
						{
							Instance->TermBody();
						}

						// Instances transforms are in global space
						const FTransform InstanceTM = FTransform(RequestFragment.Transforms[BodyIndex]);

						// Skip bodies if associated transform is not valid (e.g., scale too small, NaN, mirroring)
						const bool bTransformIsValid = Instance->ValidateTransform(InstanceTM, TEXT("batch initializer skipped that instance"), BodySetup);
						if (bTransformIsValid)
						{
							if (!Instance->GetOverrideWalkableSlopeOnInstance())
							{
								Instance->SetWalkableSlopeOverride(BodySetup->WalkableSlopeOverride, false);
							}
							InstanceBodiesSanitized.Add(Instance.Get());

							Transforms.Add(InstanceTM);
						}
					}
				}

				if (InstanceBodiesSanitized.Num() > 0)
				{
					if (RequestFragment.bDynamic)
					{
						const FInitBodySpawnParams SpawnParams(
							/*bStaticPhysics*/false,
							/*bPhysicsTypeDeterminesSimulation*/false);

						constexpr bool bCompileStatic = false;
						FInitBodiesHelper<bCompileStatic> InitBodyHelper(InstanceBodiesSanitized
							, Transforms
							, BodySetup
							, /*UPrimitiveComponent*/RequestFragment.PrimitiveComponent
							, GetWorld()->GetPhysicsScene()
							, SpawnParams
							, SpawnParams.Aggregate
							, RequestFragment.BodyInstanceOwner.Get());
						InitBodyHelper.InitBodies();
					}
					else
					{
						constexpr bool bCompileStatic = true;
						FInitBodiesHelper<bCompileStatic> InitBodiesHelper(InstanceBodiesSanitized
							, Transforms
							, BodySetup
							, /*UPrimitiveComponent*/RequestFragment.PrimitiveComponent
							, GetWorld()->GetPhysicsScene()
							, FInitBodySpawnParams(/*UPrimitiveComponent*/RequestFragment.PrimitiveComponent)
							, FPhysicsAggregateHandle()
							, RequestFragment.BodyInstanceOwner.Get());
						InitBodiesHelper.InitBodies();
					}

					UE_VLOG_UELOG(this, LogMassEngine, Log, TEXT("Initialized %d %s body instances using mesh: %s")
						, NumBodies
						, CommonBodyType == EBodyType::SimulatedDynamic ? TEXT("simulated dynamic") : CommonBodyType == EBodyType::NonSimulatedDynamic ? TEXT("non-simulated dynamic") : TEXT("static")
						, *GetPathNameSafe(Mesh));

					const TArray<TSharedPtr<FChaosUserDefinedEntity>>& UserDefinedEntityList = RequestFragment.ChaosUserDefinedEntity;
					if (RequestFragment.ChaosUserDefinedEntity.Num() > 0)
					{
						const bool bSharedUserDefinedEntity = UserDefinedEntityList.Num() == 1 && RequestFragment.InstanceBodies.Num() > 1;
						if (bSharedUserDefinedEntity)
						{
							TArray<Chaos::FPhysicsObject*> PhysicsObjects;
							PhysicsObjects.Reserve(RequestFragment.InstanceBodies.Num());
							for (const TSharedPtr<FBodyInstance>& InstancedBody : RequestFragment.InstanceBodies)
							{
								if (Chaos::FPhysicsObject* PhysicsObject = InstancedBody && InstancedBody->GetPhysicsActor() ? InstancedBody->GetPhysicsActor()->GetPhysicsObject() : nullptr)
								{
									PhysicsObjects.Add(PhysicsObject);
								}
							}
							FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects)->SetUserDefinedEntity(PhysicsObjects, UserDefinedEntityList[0].Get());
						}
						else if (ensureMsgf(UserDefinedEntityList.Num() == RequestFragment.InstanceBodies.Num()
							, TEXT("UserDefinedEntity should be shared or its count should match the number of body instances")))
						{
							FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(GetWorld()->GetPhysicsScene());
							int32 Index = 0;
							for (const TSharedPtr<FBodyInstance>& InstancedBody : RequestFragment.InstanceBodies)
							{
								if (UserDefinedEntityList[Index])
								{
									if (Chaos::FPhysicsObject* PhysicsObject = InstancedBody && InstancedBody->GetPhysicsActor() ? InstancedBody->GetPhysicsActor()->GetPhysicsObject() : nullptr)
									{
										Interface->SetUserDefinedEntity({PhysicsObject}, UserDefinedEntityList[Index].Get());
									}
								}
								Index++;
							}
						}
					}
				}
			}


			Context.Defer().DestroyEntities(EntitiesToDestroy);
		});
}

//----------------------------------------------------------------------//
// UMassPhysicsBodyInstanceReInitializer
//----------------------------------------------------------------------//
UMassPhysicsBodyInstanceReInitializer::UMassPhysicsBodyInstanceReInitializer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EntityQuery(*this)
{
	ObservedTypes.Append({
		FMassPhysicsDynamicBodyTag::StaticStruct()
		, FMassPhysicsSimulatedBodyTag::StaticStruct()
		, FMassPhysicsCollisionSettingsFragment::StaticStruct() });
	ObservedOperations = EMassObservedOperationFlags::All;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllWorldModes);
}

void UMassPhysicsBodyInstanceReInitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddTagRequirement<FMassPhysicsBodyInstanceInitializedTag>(EMassFragmentPresence::All);
}

void UMassPhysicsBodyInstanceReInitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	EntityQuery.ForEachEntityChunk(ExecutionContext, [this](const FMassExecutionContext& Context)
		{
			Context.GetEntityManagerChecked().Defer().PushCommand<FMassCommandRemoveElements<FMassPhysicsBodyInstanceInitializedTag>>(Context.GetEntities());
		});
}
