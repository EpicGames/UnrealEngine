// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassObserverProcessor.h"
#include "MassProcessor.h"
#include "MassPhysicsProcessors.generated.h"

struct FMassEntityManager;
struct FMassExecutionContext;

/**
 * Processor responsible for batching individual BodyInstance owner into a single initialization request.
 */
UCLASS()
class UMassPhysicsBodyInstancesBatcher : public UMassProcessor
{
	GENERATED_BODY()

public:
	explicit UMassPhysicsBodyInstancesBatcher(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext) override;

	FMassArchetypeHandle BatchInitializationRequestArchetype;
	FMassEntityQuery EntityQuery;
};

/**
 * Processor responsible for initializing batches of FBodyInstances.
 */
UCLASS()
class UMassPhysicsBodyInstancesBatchInitializer : public UMassProcessor
{
	GENERATED_BODY()

public:
	explicit UMassPhysicsBodyInstancesBatchInitializer(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:

	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext) override;

	FMassEntityQuery EntityQuery;
};

/**
 * Processor responsible to monitor changes to body instances configurations to trigger a re-initialization of the instance.
 */
UCLASS()
class UMassPhysicsBodyInstanceReInitializer : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	explicit UMassPhysicsBodyInstanceReInitializer(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:

	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext) override;

	FMassEntityQuery EntityQuery;
};