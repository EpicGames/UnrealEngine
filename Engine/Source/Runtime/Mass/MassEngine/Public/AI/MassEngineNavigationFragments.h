// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityElementTypes.h"
#include "AI/Navigation/NavigationElement.h"

#include "MassEngineNavigationFragments.generated.h"


/**
 * Shared Fragment holding the properties defining how a given entity should affect navigation data
 */
USTRUCT()
struct FMassNavigationRelevantParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	/** If set, navmesh will not be generated under the surface of the geometry */
	UPROPERTY()
	bool bFillCollisionUnderneathForNavData = false;
};

/**
 * Tag indicating that the entity has been evaluated for navigation relevance.
 * Added to all entities after initial evaluation regardless of relevance outcome,
 * so that non-relevant entities are not re-evaluated every frame.
 */
USTRUCT()
struct FMassNavigationEvaluatedTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Fragment holding the registration handle to the navigation element created from a Mass entity.
 * The fragment is added to indicate that a Mass entity is relevant to the AI navigation system.
 */
USTRUCT()
struct FMassNavigationRelevantFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassNavigationRelevantFragment() = default;
	explicit FMassNavigationRelevantFragment(const FNavigationElementHandle Handle)
		: Handle(Handle)
	{
	}

	/** Handle to the Navigation element created and registered for the entity */
	FNavigationElementHandle Handle;
};
