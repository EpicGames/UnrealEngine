// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassObserverProcessor.h"
#include "MassSignalProcessorBase.h"
#include "AI/MassEngineNavigationFragments.h"

#include "MassEngineNavigationProcessors.generated.h"

struct FMassEntityManager;
struct FMassExecutionContext;

/**
 * Processor responsible for registering new NavigationElement representing Mass entities that are relevant to the AI navigation system
 * and adding the FMassNavigationRelevantFragment fragment to the entities.
 */
UCLASS()
class UMassNavigationElementProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UMassNavigationElementProcessor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery NewRelevantEntityQuery;
};

/**
 * Processor responsible for updating NavigationElement representing Mass entities that are relevant to the AI navigation system
 * when one of its core property gets modified (e.g., transform, mesh, navigation bounds, etc.)
 * This processor subscribes to the following signals to perform an update:
 * - UE::Mass::Signals::TransformChanged
 * - UE::Mass::Signals::MeshChanged
 */
UCLASS()
class UMassDirtyNavigationRelevantUpdateProcessor : public UMassSignalProcessorBase
{
	GENERATED_BODY()

public:
	UMassDirtyNavigationRelevantUpdateProcessor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup& EntitySignals) override;
};

/**
 * Processor responsible for unregistering the NavigationElements when Mass entities are destroyed or no longer relevant
 * to the AI navigation system.
 */
UCLASS()
class UMassNavigationRelevantMassDeinitializer : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UMassNavigationRelevantMassDeinitializer();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
	FMassEntityQuery NonRelevantEntityQuery;
};