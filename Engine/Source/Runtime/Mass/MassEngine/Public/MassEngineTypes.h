// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogCategory.h"
#include "UObject/NameTypes.h"

MASSENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogMassEngine, Warning, All);

namespace UE::Mass::Signals
{
	const FName TransformChanged = FName(TEXT("TransformChanged"));
	const FName MeshChanged = FName(TEXT("MeshChanged"));
	const FName MeshVisualPropertyChanged = FName(TEXT("MeshVisualPropertyChanged"));
	const FName RenderStateDirty = FName(TEXT("RenderStateDirty"));
	
#if WITH_EDITOR
	const FName SelectionChanged = FName(TEXT("SelectionChanged"));
	const FName LevelEditingStateChanged = FName(TEXT("LevelEditingStateChanged"));
#endif // WITH_EDITOR
}

namespace UE::Mass::ProcessorGroupNames
{
	const FName SetupRenderState = FName(TEXT("SetupRenderState"));
	const FName CreateRenderState = FName(TEXT("CreateRenderState"));

	const FName SetupBatchedInitializations = FName(TEXT("SetupBatchedInitializations"));
	const FName ProcessBatchedInitializations = FName(TEXT("ProcessBatchedInitializations"));
}