// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Mass/EntityElementTypes.h"
#include "MassEntityLinkFragments.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "Elements/Framework/TypedElementHandle.h"

#include "MassEngineMeshFragments.generated.h"

struct FTransformFragment;
class URuntimeVirtualTexture;
struct FStaticMeshSceneProxyDesc;
struct FInstancedStaticMeshSceneProxyDesc;
struct FMassRenderStateHelper;

#define UE_API MASSENGINE_API

/**
 * Const shared fragment holding a pointer to the static mesh associated to the entity
 */
USTRUCT()
struct FMassStaticMeshFragment : public FMassConstSharedFragment
{
	GENERATED_BODY()

	FMassStaticMeshFragment() = default;
	UE_API explicit FMassStaticMeshFragment(TNotNull<const UStaticMesh*> InMesh);

	UPROPERTY(VisibleAnywhere, Category=Debug)
	TWeakObjectPtr<const UStaticMesh> Mesh;
};

/**
 * Const shared fragment holding all the visualization flags for a mesh
 */
USTRUCT()
struct FMassVisualizationMeshFragment : public FMassConstSharedFragment
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Mesh)
	int64 CustomDepthStencilValue = 0;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	int32 TranslucencySortPriority = 0;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	FLightingChannels LightingChannels;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 HiddenInGame : 1 = false;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 CastShadow : 1 = true;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 CastShadowAsTwoSided : 1 = false;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 CastHiddenShadow : 1 = false;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 AffectDynamicIndirectLighting : 1 = true;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 AffectIndirectLightingWhileHidden : 1 = false;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 ReceivesDecals : 1 = true;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 NeverDistanceCull : 1 = false;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 CastFarShadow : 1 = false;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 CastInsetShadow : 1 = false;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 CastContactShadow : 1 = false;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 RenderCustomDepth : 1 = false;
};

/**
 * Const shared fragment holding the editor visualization flags for a mesh
 */
USTRUCT()
struct FMassEditorVisualizationMeshFragment : public FMassConstSharedFragment
{
	GENERATED_BODY()

#if  WITH_EDITORONLY_DATA
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 bShouldRenderSelected : 1 = false;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 bHiddenInEditor : 1 = false;
	UPROPERTY(VisibleAnywhere, Category = Mesh)
	uint8 bLevelInstanceEditingState : 1 = false;
#endif //  WITH_EDITORONLY_DATA
};

/**
 * Const shared fragment holding the list of material override for a mesh
 */
USTRUCT()
struct FMassOverrideMaterialsFragment : public FMassConstSharedFragment
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Debug)
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;
};

template<>
struct TMassFragmentTraits<FMassVisualizationMeshFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

/**
 * Fragment holding rendering information of all the different primitives
 */
USTRUCT()
struct FMassRenderPrimitiveFragment : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY(Transient, VisibleAnywhere, Category = Debug)
	FBoxSphereBounds LocalBounds = {};
	UPROPERTY(Transient, VisibleAnywhere, Category = Debug)
	FBoxSphereBounds WorldBounds = {}; // Maybe not here...
	UPROPERTY(Transient, VisibleAnywhere, Category = Debug)
	FCustomPrimitiveData CustomPrimitiveData;
	UPROPERTY(Transient, VisibleAnywhere, Category = Debug)
	TEnumAsByte<enum EDetailMode> DetailMode = EDetailMode::DM_Low;
	UPROPERTY(Transient, VisibleAnywhere, Category = Debug)
	uint8 bIsVisible : 1 = true;
	UPROPERTY(Transient, VisibleAnywhere, Category = Debug)
	uint8 bRasterizeAsFilledConvexVolume : 1 = false;
	UPROPERTY(Transient, VisibleAnywhere, Category = Debug)
	TArray<TObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;
};

template<>
struct TMassFragmentTraits<FMassRenderPrimitiveFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};


/**
 * Fragment holding the rendering information for a static mesh
 */
USTRUCT()
struct FMassRenderStaticMeshFragment : public FMassFragment
{
	GENERATED_BODY()

	UE_API FMassRenderStaticMeshFragment();
	UE_API explicit FMassRenderStaticMeshFragment(const TSharedRef<FStaticMeshSceneProxyDesc>& InStaticMeshSceneProxyDesc);

	UStaticMesh* GetStaticMesh();
	const UStaticMesh* GetStaticMesh() const;

	TSharedPtr<FStaticMeshSceneProxyDesc> StaticMeshSceneProxyDesc = nullptr;
};

template<>
struct TMassFragmentTraits<FMassRenderStaticMeshFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

#if WITH_EDITOR
/**
 * Hit proxy to use along with the FMassTypeElementFragment
 */
struct HTypeElementHandleHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	HTypeElementHandleHitProxy(const FTypedElementHandle& InElementHandle) :
		HHitProxy(HPP_World),
		ElementHandle(InElementHandle)
	{
	}

	HTypeElementHandleHitProxy(const FTypedElementHandle& InElementHandle, int32 InMaterialIndex, int32 InSectionIndex) :
		HHitProxy(HPP_World),
		ElementHandle(InElementHandle),
		SectionIndex(InSectionIndex),
		MaterialIndex(InMaterialIndex)
	{
	}

	FTypedElementHandle ElementHandle;
	int32 SectionIndex = -1;
	int32 MaterialIndex = -1;

	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return false;
	}

	virtual FTypedElementHandle GetElementHandle() const override
	{
		return ElementHandle;
	}

	virtual EMouseCursor::Type GetMouseCursor() override;
};
#endif // WITH_EDITOR

/**
 * Fragment holding rendering information for a instanced static mesh
 */
USTRUCT()
struct FMassRenderISMFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassRenderISMFragment();
	explicit FMassRenderISMFragment(const TSharedRef<FInstancedStaticMeshSceneProxyDesc>& InInstancedStaticMeshSceneProxyDesc);

	UStaticMesh* GetStaticMesh();
	const UStaticMesh* GetStaticMesh() const;

#if WITH_EDITOR
	TSparseArray<TRefCountPtr<HTypeElementHandleHitProxy>> PerInstanceHitProxy;
#endif // WITH_EDITOR
	TSparseArray<FInstancedStaticMeshInstanceData> PerInstanceSMData;
	TArray<float> PerInstanceSMCustomData;
	TArray<FInstancedStaticMeshRandomSeed> AdditionalRandomSeeds;

	TSharedPtr<FInstancedStaticMeshSceneProxyDesc> InstancedStaticMeshSceneProxyDesc = nullptr;
	int32 InstancingRandomSeed = 0;
};

template<>
struct TMassFragmentTraits<FMassRenderISMFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};


/**
 * Fragment holding the instanced static mesh entity for the instances
 */
USTRUCT()
struct FMassRenderISMLinkFragment : public FMassEntityLinkFragment
{
	GENERATED_BODY()
};

/**
 * Fragment holding the index for the instances of an instanced static mesh 
 */
USTRUCT()
struct FMassRenderISMInstanceFragment : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY(Transient, VisibleAnywhere, Category = Debug)
	int32 InstanceIndex = -1;
#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, VisibleAnywhere, Category = Debug)
	int32 HitProxyIndex = -1;
#endif // WITH_EDITORONLY_DATA
};

/**
 * Fragment hold the type element for selection
 */
USTRUCT()
struct FMassTypeElementFragment : public FMassFragment
{
	GENERATED_BODY()

	FTypedElementHandle TypeElementHandle;
};

template<>
struct TMassFragmentTraits<FMassTypeElementFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

/**
 * Tag to tell if the SceneProxy was created
 */
USTRUCT()
struct FMassSceneProxyCreatedTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Tag to tell if the static mesh is a candidate for instantiation
 */
USTRUCT()
struct FMassRenderISMCandidateTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Optional const shared fragment for ISM grouping scoping.
 * When present, its GroupId is included in the ISM hash, ensuring entities
 * with different GroupIds get separate ISM entities even if they share the
 * same mesh/vis flags. Use cases: per-cell scoping, per-level scoping, etc.
 */
USTRUCT()
struct FMassISMGroupingFragment : public FMassConstSharedFragment
{
	GENERATED_BODY()

	FMassISMGroupingFragment() = default;
	explicit FMassISMGroupingFragment(const FGuid& InGroupId)
		: GroupId(InGroupId)
	{}

	UPROPERTY(VisibleAnywhere, Category = Debug)
	FGuid GroupId;
};

#undef UE_API
