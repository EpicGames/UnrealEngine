// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotNull.h"

struct FPrimitiveSceneProxyDesc;
struct FInstancedStaticMeshSceneProxyDesc;
struct FMassEditorVisualizationMeshFragment;
struct FMassVisualizationMeshFragment;
struct FMassStaticMeshFragment;
struct FStaticMeshSceneProxyDesc;
class UActorComponent;
struct FTransformFragment;
struct FMassRenderPrimitiveFragment;
struct FMassRenderStaticMeshFragment;
struct FMassRenderISMFragment;

#define UE_API MASSENGINE_API

namespace UE::MassEngine::Mesh
{
	/**
	 * Initializes the transform, primitive fragments from a UActorComponent
	 * @param Component to initialize the fragments from
	 * @param TransformFragment to set up
	 * @param RenderPrimitiveFragment to set up
	 */
	UE_API void InitializePrimitiveFragmentsFromComponent(TNotNull<const UActorComponent*> Component, FTransformFragment& TransformFragment, FMassRenderPrimitiveFragment& RenderPrimitiveFragment);

	/**
	 * Initializes the render transform, primitive, render static mesh fragments from a UActorComponent
	 * @param Component to initialize the fragments from
	 * @param TransformFragment the transform fragment to set up
	 * @param RenderPrimitiveFragment the render primitive fragment to set up
	 * @param RenderStaticMeshFragment the render static mesh fragment to set up
	 */
	UE_API void InitializeStaticMeshFragmentsFromComponent(TNotNull<const UActorComponent*> Component, FTransformFragment& TransformFragment, FMassRenderPrimitiveFragment& RenderPrimitiveFragment, FMassRenderStaticMeshFragment& RenderStaticMeshFragment);

	/**
	 * Initializes the render transform, primitive, render instanced static mesh fragments from a UActorComponent
	 * @param Component to initialize the fragments from
	 * @param TransformFragment the transform fragment to set up
	 * @param RenderPrimitiveFragment the render primitive fragment to set up
	 * @param RenderISMFragment the render instanced static mesh fragment to set up
	 */
	UE_API void InitializeISMFragmentsFromComponent(TNotNull<const UActorComponent*> Component, FTransformFragment& TransformFragment, FMassRenderPrimitiveFragment& RenderPrimitiveFragment, FMassRenderISMFragment& RenderISMFragment);

	enum class EBoundsType
	{
		LocalBounds,
		WorldBounds,
		NavigationBounds
	};

	/**
	 * Calculates the bounds of the instanced static mesh from the mass fragments
	 * @param TransformFragment Transform to use as a root of the ISM
	 * @param RenderISMFragment List of instances transform of the ISM
	 * @param BoundsType Bounds type to calculate
	 * @return 
	 */
	UE_API FBoxSphereBounds CalculateInstancedStaticMeshBounds(const FTransformFragment& TransformFragment, const FMassRenderISMFragment& RenderISMFragment, const EBoundsType BoundsType);

#if WITH_EDITOR
	/**
	 * Initialize the scene proxy desc from the editor mesh fragments
	 * @param EditorMeshFragment fragment describing the editor mesh parameters
	 * @param PrimitiveSceneProxyDesc scene proxy desc to initialize
	 */
	void InitializePrimitiveSceneProxyDescFromEditorFragment(const FMassEditorVisualizationMeshFragment& EditorMeshFragment, FPrimitiveSceneProxyDesc& PrimitiveSceneProxyDesc);
#endif // WITH_EDITOR

	/**
	 * Initialize the static mesh scene proxy desc from mass fragments
	 * @param StaticMeshFragment fragment describing the static mesh
	 * @param MeshFragment fragment describing the mesh parameters
	 * @param TransformFragment fragment specifying the transform
	 * @param StaticMeshSceneProxyDesc scene proxy desc to initialize
	 */
	void InitializeStaticMeshSceneProxyDescFromFragment(const FMassStaticMeshFragment& StaticMeshFragment, const FMassVisualizationMeshFragment& MeshFragment, const FTransformFragment& TransformFragment, FStaticMeshSceneProxyDesc& StaticMeshSceneProxyDesc);

	/**
	 * Initialize the instanced static mesh scene proxy desc from mass fragments
	 * @param StaticMeshFragment fragment describing the static mesh
	 * @param MeshFragment fragment describing the mesh parameters
	 * @param TransformFragment fragment specifying the transform
	 * @param ISMSceneProxyDesc scene proxy desc to initialize
	 */
	void InitializeInstanceStaticMeshSceneProxyDescFromFragment(const FMassStaticMeshFragment& StaticMeshFragment, const FMassVisualizationMeshFragment& MeshFragment, const FTransformFragment& TransformFragment, FInstancedStaticMeshSceneProxyDesc& ISMSceneProxyDesc);


}

#undef UE_API