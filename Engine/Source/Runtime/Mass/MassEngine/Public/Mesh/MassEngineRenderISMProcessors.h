// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mesh/MassEngineRenderStaticMeshProcessors.h"
#include "MassObserverProcessor.h"
#include "MassSignalProcessorBase.h"
#include "MassArchetypeTypes.h"

#include "MassEngineRenderISMProcessors.generated.h"

struct FMassEntityManager;
struct FMassExecutionContext;

/**
 * Processor responsible for creating instanced static mesh entity and setting up its render state for the renderer
 */
UCLASS()
class UMassInstantiateStaticMeshAndCreateISMRenderStateProcessor : public UMassBaseStaticMeshSetupRenderStateProcessor
{
	GENERATED_BODY()

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext) override;

	FMassElementBitSet ISMEntityComposition;
	FMassArchetypeHandle ISMEntityArchetype;
	FMassElementBitSet ISMEntityCompositionWithMaterialOverrides;
	FMassArchetypeHandle ISMEntityArchetypeWithMaterialOverrides;

#if WITH_EDITOR
	FMassElementBitSet ISMEntityCompositionWithEditorMesh;
	FMassArchetypeHandle ISMEntityArchetypeWithEditorMesh;
	FMassElementBitSet ISMEntityCompositionWithMaterialOverridesAndEditorMesh;
	FMassArchetypeHandle ISMEntityArchetypeWithMaterialOverridesAndEditorMesh;
#endif // WITH_EDITOR
};

/**
 * Processor responsible for notifying the linked instanced static mesh entity to update the render proxy
 * This processor listen to the following signals on the instances entities
 * - UE::Mass::Signals::TransformChanged
 * - UE::Mass::Signals::SelectionChanged
 */
UCLASS()
class UMassDirtyRenderISMInstanceProcessor : public UMassSignalProcessorBase
{
	GENERATED_BODY()

public:
	explicit UMassDirtyRenderISMInstanceProcessor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup& EntitySignals) override;
};

/**
 * Processor responsible for removing instances when the required fragments are being removed
 */
UCLASS()
class UMassISMDestroyRenderStateProcessor : public UMassBaseStaticMeshDestroyRenderStateProcessor
{
	GENERATED_BODY()

public:
	UMassISMDestroyRenderStateProcessor();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};