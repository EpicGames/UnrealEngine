// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassObserverProcessor.h"
#include "MassSignalProcessorBase.h"

#include "MassEngineRenderStateProcessors.generated.h"

struct FMassEntityManager;
struct FMassExecutionContext;

/**
 * Processor responsible for creating the renderer states
 */
UCLASS()
class UMassCreateRenderStateProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UMassCreateRenderStateProcessor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext) override;

	FMassEntityQuery EntityQuery;
};

/**
 * Processor responsible for updating or recreating the render state when dirtied
 * Listens to the following signals:
 * - UE::Mass::Signals::TransformChanged
 * - UE::Mass::Signals::MeshVisualPropertyChanged
 * - UE::Mass::Signals::MeshChanged
 * - UE::Mass::Signals::SelectionChanged
 * - UE::Mass::Signals::LevelEditingStateChanged
 * - UE::Mass::Signals::RenderStateDirty
 */
UCLASS()
class UMassRenderStateDirtyUpdateProcessor : public UMassSignalProcessorBase
{
	GENERATED_BODY()

public:
	UMassRenderStateDirtyUpdateProcessor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup& EntitySignals) override;
};

/**
 * Processor responsible for destroying the render states
 */
UCLASS()
class UMassDestroyRenderStateProcessor : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UMassDestroyRenderStateProcessor();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};