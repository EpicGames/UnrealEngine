// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassObserverProcessor.h"
#include "MassSignalProcessorBase.h"

#include "MassEngineRenderStaticMeshProcessors.generated.h"

struct FMassEntityManager;
struct FMassExecutionContext;

/**
 * Base processor responsible for setting up the static mesh render state for the renderer
 */
UCLASS(Abstract)
class UMassBaseStaticMeshSetupRenderStateProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UMassBaseStaticMeshSetupRenderStateProcessor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	FMassEntityQuery EntityQuery;
};

/**
 * Processor responsible for setting up the static mesh render state for the renderer
 */
UCLASS()
class UMassStaticMeshSetupRenderStateProcessor : public UMassBaseStaticMeshSetupRenderStateProcessor
{
	GENERATED_BODY()

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext) override;
};

/**
 * Processor responsible for updating the static mesh render state when dirtied
 * Listening the following signals on the entity
 * - UE::Mass::Signals::TransformChanged
 * - UE::Mass::Signals::MeshVisualPropertyChanged
 * - UE::Mass::Signals::MeshChanged
 * - UE::Mass::Signals::SelectionChanged
 * - UE::Mass::Signals::LevelEditingStateChanged
 */
UCLASS()
class UMassStaticMeshUpdateRenderStateProcessor : public UMassSignalProcessorBase
{
	GENERATED_BODY()

public:
	UMassStaticMeshUpdateRenderStateProcessor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup& EntitySignals) override;
};


/**
 * Base processor responsible for removing the common static mesh render state fragments when the required fragments are being removed
 */
UCLASS(Abstract)
class UMassBaseStaticMeshDestroyRenderStateProcessor : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UMassBaseStaticMeshDestroyRenderStateProcessor();

	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager);

protected:
	FMassEntityQuery EntityQuery;
};

/**
 * Processor responsible for removing the static mesh render state fragments when the required fragments are being removed
 */
UCLASS()
class UMassStaticMeshDestroyRenderStateProcessor : public UMassBaseStaticMeshDestroyRenderStateProcessor
{
	GENERATED_BODY()

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};
