// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Mass/EntityElementTypes.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "UObject/WeakInterfacePtr.h"
#include "MassEnginePhysicsFragments.generated.h"

class FChaosUserDefinedEntity;
class IPhysicsBodyInstanceOwner;
struct FBodyInstance;

#define UE_API MASSENGINE_API

/**
 * Fragment holding data related to physics collision setup
 */
USTRUCT()
struct FMassPhysicsCollisionSettingsFragment : public FMassConstSharedFragment
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Physics)
	TEnumAsByte<ECollisionEnabled::Type> CollisionType = ECollisionEnabled::NoCollision;

	UPROPERTY(VisibleAnywhere, Category = Physics)
	FCollisionResponseContainer CollisionResponse;
};

/**
 * Fragment holding data related to a specific rigid body instance
 */
USTRUCT()
struct FMassPhysicsBodyInstanceFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassPhysicsBodyInstanceFragment() = default;

	explicit FMassPhysicsBodyInstanceFragment(const TSharedRef<FBodyInstance>& BodyInstance)
		: BodyInstance(BodyInstance)
	{
	}

	/** Weak object pointer to the owning component. Validate before using BodyInstance and ChaosUserDefinedEntity,
	 * as those shared pointers may outlive the owner and become semantically stale. */
	UPROPERTY(VisibleAnywhere, Category = Physics)
	TWeakObjectPtr<UObject> BodyInstanceOwnerObject;

	/** Shared pointer to the BodyInstance created by the owning component. */
	TSharedPtr<FBodyInstance> BodyInstance = nullptr;

	/**
	 * Optional payload to associate with the created PhysicsObject
	 * after BodyInstance initialization (i.e., FWritePhysicsObjectInterface::SetUserDefinedEntity).
	 */
	TSharedPtr<FChaosUserDefinedEntity> ChaosUserDefinedEntity = nullptr;
};

template <>
struct TMassFragmentTraits<FMassPhysicsBodyInstanceFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

/**
 * Tag that indicates that a given body should be initialized as dynamic with the possibility to be simulated in the physics scene
 * @see FMassPhysicsSimulatedBodyTag
 */
USTRUCT()
struct FMassPhysicsDynamicBodyTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Tag that indicates that a given body should be simulated in the physics scene
 */
USTRUCT()
struct FMassPhysicsSimulatedBodyTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Tag added after a body instance has been batched for initialization.
 * Entities with this tag are excluded from further batching. Removed by
 * UMassPhysicsBodyInstanceReInitializer to re-trigger the initialization cycle.
 */
USTRUCT()
struct FMassPhysicsBodyInstanceInitializedTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassPhysicsBodyInstancesInitializationRequestFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Mandatory weak pointer to the mesh used by all instances */
	UPROPERTY(Transient, VisibleAnywhere, Category = Physics)
	TWeakObjectPtr<const UStaticMesh> StaticMesh = nullptr;

	/** Mandatory list of transforms that should match the number of body instances */
	UPROPERTY(Transient, VisibleAnywhere, Category = Physics)
	TArray<FTransform> Transforms;

	/** Mandatory list of body instances to initialize */
	TArray<TSharedPtr<FBodyInstance>> InstanceBodies;

	/**
	 * Optional interface pointer that will be provided to the BodyInstance initialization
	 */
	UPROPERTY(Transient, VisibleAnywhere, Category = Physics)
	TObjectPtr<UPrimitiveComponent> PrimitiveComponent;

	/**
	 * Optional interface pointer that will be provided to the BodyInstance initialization.
	 * Stored as a TWeakInterfacePtr; check IsStale() before use.
	 */
	TWeakInterfacePtr<IPhysicsBodyInstanceOwner> BodyInstanceOwner;

	/**
	 * Optional list of user data payloads to associate to the created PhysicsObjects after the body instances get initialized.
	 * @note It is possible to provide a single payload for owners relying on body indices (i.e., InstanceBodyIndex).
	 */
	TArray<TSharedPtr<FChaosUserDefinedEntity>> ChaosUserDefinedEntity;

	/** Whether the instances are simulated/dynamic physics objects */
	UPROPERTY(Transient, VisibleAnywhere, Category = Physics)
	bool bDynamic = false;
};

template <>
struct TMassFragmentTraits<FMassPhysicsBodyInstancesInitializationRequestFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

#undef UE_API