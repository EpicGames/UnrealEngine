// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSignalProcessorBase.h"
#include "MassSignalSubsystem.h"
#include "MassArchetypeTypes.h"
#include "MassExecutionContext.h"
#include "Engine/World.h"
#include "Misc/ScopeLock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSignalProcessorBase)


UMassSignalProcessorBase::UMassSignalProcessorBase(const FObjectInitializer& ObjectInitializer)
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
}

void UMassSignalProcessorBase::BeginDestroy()
{
	if (UMassSignalSubsystem* SignalSubsystem = UWorld::GetSubsystem<UMassSignalSubsystem>(GetWorld()))
	{
		for (const FName& SignalName : RegisteredSignals)
		{
			SignalSubsystem->GetSignalDelegateByName(SignalName).RemoveAll(this);
		}
	}

	Super::BeginDestroy();
}

void UMassSignalProcessorBase::SubscribeToSignal(UMassSignalSubsystem& SignalSubsystem, const FName SignalName)
{
	check(!RegisteredSignals.Contains(SignalName));
	RegisteredSignals.Add(SignalName);
	SignalSubsystem.GetSignalDelegateByName(SignalName).AddUObject(this, &UMassSignalProcessorBase::OnSignalReceived);
}

void UMassSignalProcessorBase::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(SignalEntities);

	const int32 ProcessingFrameBufferIndex = CurrentFrameBufferIndex;
	{
		// we only need to lock the part where we change the current buffer index. Once that's done the incoming signals will end up 
		// in the other buffer
		UE::TRWScopeLock Lock(ReceivedSignalLock, SLT_Write);
		CurrentFrameBufferIndex = (CurrentFrameBufferIndex + 1) % BuffersCount;
	}
	
	FFrameReceivedSignals& ProcessingFrameBuffer = FrameReceivedSignals[ProcessingFrameBufferIndex];
	TArray<FEntitySignalRange>& ReceivedSignalRanges = ProcessingFrameBuffer.ReceivedSignalRanges;
	TArray<FMassEntityHandle>& SignaledEntities = ProcessingFrameBuffer.SignaledEntities;

	if (ReceivedSignalRanges.IsEmpty())
	{
		return;
	}

	TArray<FMassArchetypeHandle> ValidArchetypes;
	GetArchetypesMatchingOwnedQueries(EntityManager, ValidArchetypes);

	if (ValidArchetypes.Num() > 0)
	{
		// EntitySet stores unique array of entities per specified archetype.
		// FMassArchetypeEntityCollection expects an array of entities, a set is used to detect unique ones.
		struct FEntitySet
		{
			void Reset()
			{
				Entities.Reset();
			}

			FMassArchetypeHandle Archetype;
			TArray<FMassEntityHandle> Entities;
		};
		TArray<FEntitySet> EntitySets;

		for (const FMassArchetypeHandle& Archetype : ValidArchetypes)
		{
			FEntitySet& Set = EntitySets.AddDefaulted_GetRef();
			Set.Archetype = Archetype;
		}

		// SignalNameLookup has limit of how many signals it can handle at once, we'll do passes until all signals are processed.
		int32 SignalsToProcess = ReceivedSignalRanges.Num();
		while(SignalsToProcess > 0)
		{
			SignalNameLookup.Reset();

			// Convert signals with entity ids into arrays of entities per archetype.
			for (FEntitySignalRange& Range : ReceivedSignalRanges)
			{
				if (Range.bProcessed)
				{
					continue;
				}
				// Get bitflag for the signal name
				const uint64 SignalFlag = SignalNameLookup.GetOrAddSignalName(Range.SignalName);
				if (SignalFlag == 0)
				{
					// Will process that signal in a second iteration
					continue;
				}

				// Entities for this signal
				TArrayView<FMassEntityHandle> Entities(&SignaledEntities[Range.Begin], Range.End - Range.Begin);
				FEntitySet* PrevSet = &EntitySets[0];
				for (const FMassEntityHandle Entity : Entities)
				{
					// Add to set of supported archetypes. Dont process if we don't care about the type.
					FMassArchetypeHandle Archetype = EntityManager.GetArchetypeForEntity(Entity);
					FEntitySet* Set = PrevSet->Archetype == Archetype ? PrevSet : EntitySets.FindByPredicate([&Archetype](const FEntitySet& Set) { return Archetype == Set.Archetype; });
					if (Set != nullptr)
					{
						// We don't care about duplicates here, the FMassArchetypeEntityCollection creation below will handle it
						Set->Entities.Add(Entity);
						SignalNameLookup.AddSignalToEntity(Entity, SignalFlag);
						PrevSet = Set;
					}
				}

				Range.bProcessed = true;
				SignalsToProcess--;
			}

			// Execute per archetype.
			for (FEntitySet& Set : EntitySets)
			{
				if (Set.Entities.Num() > 0)
				{
					Context.SetEntityCollection(FMassArchetypeEntityCollection(Set.Archetype, Set.Entities, FMassArchetypeEntityCollection::FoldDuplicates));
					SignalEntities(EntityManager, Context, SignalNameLookup);
					Context.ClearEntityCollection();
				}
				Set.Reset();
			}
		}
	}

	ReceivedSignalRanges.Reset();
	SignaledEntities.Reset();
}

void UMassSignalProcessorBase::OnSignalReceived(FName SignalName, TConstArrayView<FMassEntityHandle> Entities)
{
	FEntitySignalRange Range;
	Range.SignalName = SignalName;

	UE::TRWScopeLock Lock(ReceivedSignalLock, SLT_Write);

	FFrameReceivedSignals& CurrentFrameBuffer = FrameReceivedSignals[CurrentFrameBufferIndex];

	Range.Begin = CurrentFrameBuffer.SignaledEntities.Num();
	CurrentFrameBuffer.SignaledEntities.Append(Entities.GetData(), Entities.Num());
	Range.End = CurrentFrameBuffer.SignaledEntities.Num();
	CurrentFrameBuffer.ReceivedSignalRanges.Add(MoveTemp(Range));
}
