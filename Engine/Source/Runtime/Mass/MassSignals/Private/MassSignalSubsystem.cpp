// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSignalSubsystem.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassCommandBuffer.h"
#include "MassSignalTypes.h"
#include "VisualLogger/VisualLogger.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSignalSubsystem)

CSV_DEFINE_CATEGORY(MassSignalsCounters, true);

void UMassSignalSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	CachedWorld = &GetWorldRef();
	checkf(CachedWorld, TEXT("UMassSignalSubsystem instances are expected to always be tied to a valid UWorld instance"));

	OverrideSubsystemTraits<UMassSignalSubsystem>(Collection);
}

void UMassSignalSubsystem::Deinitialize()
{
	CachedWorld = nullptr;
	Super::Deinitialize();
}

void UMassSignalSubsystem::Tick(float DeltaTime)
{
	// If we're in an AutoRTFM Transaction, exit the TickableObject Tick (which is top-level) and run again outside the transaction
	if (AutoRTFM::IsTransactional())
	{
		AutoRTFM::OnCommit([WeakThis = TWeakObjectPtr(this), DeltaTime]()
		{
			if (ensure(WeakThis.IsValid()))
			{
				WeakThis->Tick(DeltaTime);
			}
		});

		return;
	}

	CA_ASSUME(CachedWorld);
	const double CurrentTime = CachedWorld->GetTimeSeconds();

	UE_MT_SCOPED_WRITE_ACCESS(DelayedSignalsAccessDetector);

	for (int i = 0; i < DelayedSignals.Num();)
	{
		FDelayedSignal& DelayedSignal = DelayedSignals[i];
		if (DelayedSignal.TargetTimestamp <= CurrentTime)
		{
			SignalEntities(DelayedSignal.SignalName, MakeArrayView(DelayedSignal.Entities));
			DelayedSignals.RemoveAtSwap(i, EAllowShrinking::No);
		}
		else
		{
			i++;
		}
	}
}

TStatId UMassSignalSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UMassSignalSubsystem, STATGROUP_Tickables);
}

void UMassSignalSubsystem::SignalEntity(FName SignalName, const FMassEntityHandle Entity)
{
	checkf(Entity.IsSet(), TEXT("Expecting a valid entity to signal"));
	SignalEntities(SignalName, MakeArrayView(&Entity, 1));
}

void UMassSignalSubsystem::SignalEntities(FName SignalName, TConstArrayView<FMassEntityHandle> Entities)
{
	checkf(Entities.Num() > 0, TEXT("Expecting entities to signal"));
	const UE::MassSignal::FSignalDelegate& SignalDelegate = GetSignalDelegateByName(SignalName);
	SignalDelegate.Broadcast(SignalName, Entities);

#if CSV_PROFILER_STATS
	FCsvProfiler::RecordCustomStat(*SignalName.ToString(), CSV_CATEGORY_INDEX(MassSignalsCounters), Entities.Num(), ECsvCustomStatOp::Accumulate);
#endif

	UE_CVLOG(Entities.Num() == 1, this, LogMassSignals, Log, TEXT("Raising signal [%s] to entity [%s]"), *SignalName.ToString(), *Entities[0].DebugGetDescription());
	UE_CVLOG(Entities.Num() > 1, this, LogMassSignals, Log, TEXT("Raising signal [%s] to %d entities"), *SignalName.ToString(), Entities.Num());
}

void UMassSignalSubsystem::DelaySignalEntity(FName SignalName, const FMassEntityHandle Entity, const float DelayInSeconds)
{
	checkf(Entity.IsSet(), TEXT("Expecting a valid entity to signal"));
	DelaySignalEntities(SignalName, MakeArrayView(&Entity, 1), DelayInSeconds);
}

void UMassSignalSubsystem::DelaySignalEntities(FName SignalName, TConstArrayView<FMassEntityHandle> Entities, const float DelayInSeconds)
{
	// If you hit this ensure
	// - With another thread trying to delay signal then you can use DelaySignalEntityDeferred/DelaySignalEntitiesDeferred
	//   if you have access to a FMassExecutionContext.
	// - With the game thread executing UMassSignalSubsystem::Tick then you need to reorganize your tasks to prevent senders from executing
	//   at the same time as the subsystem tick.
	UE_MT_SCOPED_WRITE_ACCESS(DelayedSignalsAccessDetector);

	FDelayedSignal& DelayedSignal = DelayedSignals.Emplace_GetRef();
	DelayedSignal.SignalName = SignalName;
	DelayedSignal.Entities = Entities;

	check(CachedWorld);
	DelayedSignal.TargetTimestamp = CachedWorld->GetTimeSeconds() + DelayInSeconds;

	UE_CVLOG(Entities.Num() == 1, this, LogMassSignals, Log, TEXT("Delay signal [%s] to entity [%s] in %.2f"), *SignalName.ToString(), *Entities[0].DebugGetDescription(), DelayInSeconds);
	UE_CVLOG(Entities.Num() > 1,this, LogMassSignals, Log, TEXT("Delay signal [%s] to %d entities in %.2f"), *SignalName.ToString(), Entities.Num(), DelayInSeconds);
}

void UMassSignalSubsystem::SignalEntityDeferred(FMassExecutionContext& Context, FName SignalName, const FMassEntityHandle Entity)
{
	checkf(Entity.IsSet(), TEXT("Expecting a valid entity to signal"));
	SignalEntitiesDeferred(Context, SignalName, MakeArrayView(&Entity, 1));
}

void UMassSignalSubsystem::SignalEntitiesDeferred(FMassExecutionContext& Context, FName SignalName, TConstArrayView<FMassEntityHandle> Entities)
{
	checkf(Entities.Num() > 0, TEXT("Expecting entities to signal"));
	Context.Defer().PushCommand<FMassDeferredSetCommand>([SignalName, InEntities = TArray<FMassEntityHandle>(Entities)](const FMassEntityManager& InOutEntityManager)
	{
		UMassSignalSubsystem* SignalSubsystem = UWorld::GetSubsystem<UMassSignalSubsystem>(InOutEntityManager.GetWorld());
		SignalSubsystem->SignalEntities(SignalName, InEntities);
	});

	UE_CVLOG(Entities.Num() == 1, this, LogMassSignals, Log, TEXT("Raising deferred signal [%s] to entity [%s]"), *SignalName.ToString(), *Entities[0].DebugGetDescription());
	UE_CVLOG(Entities.Num() > 1, this, LogMassSignals, Log, TEXT("Raising deferred signal [%s] to %d entities"), *SignalName.ToString(), Entities.Num());
}

void UMassSignalSubsystem::DelaySignalEntityDeferred(FMassExecutionContext& Context, FName SignalName, const FMassEntityHandle Entity, const float DelayInSeconds)
{
	checkf(Entity.IsSet(), TEXT("Expecting a valid entity to signal"));
	DelaySignalEntitiesDeferred(Context, SignalName, MakeArrayView(&Entity, 1), DelayInSeconds);
}

void UMassSignalSubsystem::DelaySignalEntitiesDeferred(FMassExecutionContext& Context, FName SignalName, TConstArrayView<FMassEntityHandle> Entities, const float DelayInSeconds)
{
	checkf(Entities.Num() > 0, TEXT("Expecting entities to signal"));
	Context.Defer().PushCommand<FMassDeferredSetCommand>([SignalName, InEntities = TArray<FMassEntityHandle>(Entities), DelayInSeconds](const FMassEntityManager& InOutEntityManager)
	{
		UMassSignalSubsystem* SignalSubsystem = UWorld::GetSubsystem<UMassSignalSubsystem>(InOutEntityManager.GetWorld());
		SignalSubsystem->DelaySignalEntities(SignalName, InEntities, DelayInSeconds);
	});

	UE_CVLOG(Entities.Num() == 1, this, LogMassSignals, Log, TEXT("Delay deferred signal [%s] to entity [%s] in %.2f"), *SignalName.ToString(), *Entities[0].DebugGetDescription(), DelayInSeconds);
	UE_CVLOG(Entities.Num() > 1,this, LogMassSignals, Log, TEXT("Delay deferred signal [%s] to %d entities in %.2f"), *SignalName.ToString(), Entities.Num(), DelayInSeconds);
}
