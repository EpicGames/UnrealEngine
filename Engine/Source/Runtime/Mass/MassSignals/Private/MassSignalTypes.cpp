// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassSignalTypes.h"
#include "MassEntityManager.h"

uint64 FMassSignalNameLookup::GetOrAddSignalName(const FName SignalName)
{
	int32 Index = SignalNames.Find(SignalName);
	if (Index == INDEX_NONE)
	{
		if (SignalNames.Num() >= MaxSignalNames)
		{
			return 0;
		}
		Index = SignalNames.Add(SignalName);
	}
	return 1ULL << Index;
}

void FMassSignalNameLookup::AddSignalToEntity(const FMassEntityHandle Entity, const uint64 SignalFlag)
{
	uint64& Signals = EntitySignals.FindOrAdd(Entity, 0);
	Signals |= SignalFlag;
}

void FMassSignalNameLookup::GetSignalsForEntity(const FMassEntityHandle Entity, TArray<FName>& OutSignals) const
{
	OutSignals.Reset();
	if (const uint64* Signals = EntitySignals.Find(Entity))
	{
		const uint64 Start = FMath::CountTrailingZeros64(*Signals);
		const uint64 End = FMath::CountLeadingZeros64(*Signals);
		for (uint64 i = Start; i < 64 - End; i++)
		{
			const uint64 SignalFlag = 1ULL << i;
			if ((*Signals) & SignalFlag)
			{
				OutSignals.Add(SignalNames[(uint32)i]);
			}
		}
	}
}

void FMassSignalNameLookup::Reset()
{
	SignalNames.Reset();
	EntitySignals.Reset();
}