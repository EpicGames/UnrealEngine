// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassEntityManager.h"
#include "MassSignalTypes.h"
#include "Containers/StaticArray.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "MassEntityQuery.h"
#include "MassSignalProcessorBase.generated.h"

#define UE_API MASSSIGNALS_API

class UMassSignalSubsystem;

/** 
 * Processor for executing signals on each targeted entities
 * The derived classes only need to implement the method SignalEntities to actually received the raised signals for the entities they subscribed to 
 */
UCLASS(MinimalAPI, abstract)
class UMassSignalProcessorBase : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassSignalProcessorBase(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

protected:
	UE_API virtual void BeginDestroy() override;

	/** Configure the owned FMassEntityQuery instances to express processor queries requirements */
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>&) override {}

	/**
	 * Actual method that derived class needs to implement to act on a signal that is raised for that frame
	 * @param EntitySubsystem is the system to execute the lambdas on each entity chunk
	 * @param Context is the execution context to be passed when executing the lambdas
	 * @param EntitySignals Look up to retrieve for each entities their raised signal via GetSignalsForEntity
	 */
	 virtual void SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup& EntitySignals) PURE_VIRTUAL(UMassSignalProcessorBase::SignalEntities, );

	/**
	 * Callback that is being called when new signal is raised
	 * @param SignalName is the name of the signal being raised
	 * @param Entities are the targeted entities for this signal
	 */
	 UE_API virtual void OnSignalReceived(FName SignalName, TConstArrayView<FMassEntityHandle> Entities);

	/**
	 * Execution method for this processor
	 * @param EntitySubsystem is the system to execute the lambdas on each entity chunk
	 * @param Context is the execution context to be passed when executing the lambdas
	 */
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	/** 
	 * We don't want signaling processors to be pruned by default, since that can mess up signal processing 
	 * just after processor's creation (might miss some signals initially).
	 */
	virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode = true) const override { return false; }


	/**
	 * To receive notification about a particular signal, you need to subscribe to it.
	 * @param SignalName is the name of the signal to receive notification about
	 */
	UE_API void SubscribeToSignal(UMassSignalSubsystem& SignalSubsystem, const FName SignalName);

	FMassEntityQuery EntityQuery;

private:

	/** Stores a range of indices in the SignaledEntities TArray of Entities and the associated signal name */
	struct FEntitySignalRange
	{
		FName SignalName;
		int32 Begin = 0;
		int32 End = 0;
		bool bProcessed = false;
	};

	struct FFrameReceivedSignals
	{
		/** Received signals are double buffered as we can receive new one while processing them */
		TArray<FEntitySignalRange> ReceivedSignalRanges;

		/** the list of all signaled entities, can contain duplicates */
		TArray<FMassEntityHandle> SignaledEntities;
	};

	static constexpr int BuffersCount = 2;

	/** Double buffer frame received signal as we can receive new signals as we are processing them */
	TStaticArray<FFrameReceivedSignals, BuffersCount> FrameReceivedSignals;
	
	/** Current frame buffer index of FrameReceivedSignals */
	int32 CurrentFrameBufferIndex = 0;

	/** Lookup used to store and retrieve signals per entity, only used during processing */
	FMassSignalNameLookup SignalNameLookup;

	/** List of all the registered signal names*/
	TArray<FName> RegisteredSignals;

	FTransactionallySafeRWLock ReceivedSignalLock;
};

#undef UE_API
