// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityManager.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Mass/EntityHandle.h"
#include "MassSubsystemBase.h"
#include "Misc/MTAccessDetector.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "MassSignalSubsystem.generated.h"

struct FMassExecutionContext;

namespace UE::MassSignal 
{
	DECLARE_MULTICAST_DELEGATE_TwoParams(FSignalDelegate, FName /*SignalName*/, TConstArrayView<FMassEntityHandle> /*Entities*/);
} // UE::MassSignal

/**
* A subsystem for handling Signals in Mass
*/
UCLASS(MinimalAPI)
class UMassSignalSubsystem : public UMassTickableSubsystemBase
{
	GENERATED_BODY()
	
public:

	/** 
	 * Retrieve the delegate dispatcher from the signal name
	 * @param SignalName is the name of the signal to get the delegate dispatcher from
	 */
	UE::MassSignal::FSignalDelegate& GetSignalDelegateByName(FName SignalName)
	{
		return NamedSignals.FindOrAdd(SignalName);
	}

	/**
	 * Inform a single entity of a signal being raised
	 * @param SignalName is the name of the signal raised
	 * @param Entity entity that should be informed that signal 'SignalName' was raised
	 */
	MASSSIGNALS_API void SignalEntity(FName SignalName, const FMassEntityHandle Entity);

	/**
	 * Inform multiple entities of a signal being raised
	 * @param SignalName is the name of the signal raised
	 * @param Entities list of entities that should be informed that signal 'SignalName' was raised
	 */
	MASSSIGNALS_API void SignalEntities(FName SignalName, TConstArrayView<FMassEntityHandle> Entities);

	/**
	 * Inform a single entity of a signal being raised in a certain amount of seconds
	 * @param SignalName is the name of the signal raised
	 * @param Entity entity that should be informed that signal 'SignalName' was raised
	 * @param DelayInSeconds is the amount of time before signaling the entity
	 */
	MASSSIGNALS_API void DelaySignalEntity(FName SignalName, const FMassEntityHandle Entity, const float DelayInSeconds);

 	/**
	 * Inform multiple entities of a signal being raised in a certain amount of seconds
	 * @param SignalName is the name of the signal raised
	 * @param Entities being informed of the raised signal
	 * @param DelayInSeconds is the amount of time before signaling the entities
	 */
	MASSSIGNALS_API void DelaySignalEntities(FName SignalName, TConstArrayView<FMassEntityHandle> Entities, const float DelayInSeconds);

	/**
	 * Inform single entity of a signal being raised asynchronously using the Mass Command Buffer
	 * @param Context is the Entity System execution context to push the async command
	 * @param SignalName is the name of the signal raised
	 * @param Entity entity that should be informed that signal 'SignalName' was raised
	 */
	MASSSIGNALS_API void SignalEntityDeferred(FMassExecutionContext& Context, FName SignalName, const FMassEntityHandle Entity);

	/**
	 * Inform multiple entities of a signal being raised asynchronously using the Mass Command Buffer
	 * @param Context is the Entity System execution context to push the async command
	 * @param SignalName is the name of the signal raised
	 * @param Entities list of entities that should be informed that signal 'SignalName' was raised
	 */
	MASSSIGNALS_API void SignalEntitiesDeferred(FMassExecutionContext& Context, FName SignalName, TConstArrayView<FMassEntityHandle> Entities);

 	/**
	 * Inform single entity of a signal being raised asynchronously using the Mass Command Buffer
	 * @param Context is the Entity System execution context to push the async command
	 * @param SignalName is the name of the signal raised
	 * @param Entity entity that should be informed that signal 'SignalName' was raised
	 * @param DelayInSeconds is the amount of time before signaling the entities
	 */
	MASSSIGNALS_API void DelaySignalEntityDeferred(FMassExecutionContext& Context, FName SignalName, const FMassEntityHandle Entity, const float DelayInSeconds);

 	/**
	 * Inform multiple entities of a signal being raised asynchronously using the Mass Command Buffer
	 * @param Context is the Entity System execution context to push the async command
	 * @param SignalName is the name of the signal raised
	 * @param Entities being informed of that signal was raised
	 * @param DelayInSeconds is the amount of time before signaling the entities
	 */
	MASSSIGNALS_API void DelaySignalEntitiesDeferred(FMassExecutionContext& Context, FName SignalName, TConstArrayView<FMassEntityHandle> Entities, const float DelayInSeconds);

protected:
	// USubsystem implementation Begin
	MASSSIGNALS_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	MASSSIGNALS_API virtual void Deinitialize() override;
	// USubsystem implementation End	

	MASSSIGNALS_API virtual void Tick(float DeltaTime) override;
	MASSSIGNALS_API virtual TStatId GetStatId() const override;

	/** Multithreading access detector to validate accesses to the list of delayed signals */
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(DelayedSignalsAccessDetector);

	TMap<FName, UE::MassSignal::FSignalDelegate> NamedSignals;

	struct FDelayedSignal
	{
		FName SignalName;
		TArray<FMassEntityHandle> Entities;
		double TargetTimestamp;
	};

	TArray<FDelayedSignal> DelayedSignals;

	UPROPERTY(transient)
	TObjectPtr<UWorld> CachedWorld;
};

template<>
struct TMassExternalSubsystemTraits<UMassSignalSubsystem> final
{
	enum
	{
		GameThreadOnly = false,
		// @todo this subsystem not being thread-safe when writing is an obstacle in
		// parallelizing multiple processors
		ThreadSafeWrite = false,
	};
};
