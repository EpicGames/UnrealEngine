// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Logging/LogMacros.h"
#include "MassEntityManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMassSignals, Log, All)

/**
 * MassSignalNameLookup stores list of Signal names for each entity. The names are stored per entity as a bitmask,
 * you can allocate new name using GetOrAddSignalName(). This limits the names to 64.
 */
struct FMassSignalNameLookup
{
	/** Max number of names each entity can contain */
	static constexpr int32 MaxSignalNames = 64;

	/** 
	 * Retrieve if it is already registered or adds new signal to the lookup and return the bitflag for that Signal
	 * @SignalName is the name of the signal to retrieve or add to the lookup.
	 * @return bitflag describing the name, or 0 if max names is reached. 
	 */
	MASSSIGNALS_API uint64 GetOrAddSignalName(const FName SignalName);

	/**
	 * Adds specified Signal name bitflag to an entity 
	 * @param Entity is the entity where the signal has been raised
	 * @param SignalFlag is the actual bitflag describing the signal
	 */
	MASSSIGNALS_API void AddSignalToEntity(const FMassEntityHandle Entity, const uint64 SignalFlag);

	/** 
	 * Retrieve for a specific entity the raised signal this frame
	 * @return Array of signal names raised for this entity 
	 */
	MASSSIGNALS_API void GetSignalsForEntity(const FMassEntityHandle Entity, TArray<FName>& OutSignals) const;

	/** Empties the name lookup and entity signals */
	MASSSIGNALS_API void Reset();

protected:
	/** Array of Signal names */
	TArray<FName> SignalNames;

	/** Map from entity id to name bitmask */
	TMap<FMassEntityHandle, uint64> EntitySignals;
};
